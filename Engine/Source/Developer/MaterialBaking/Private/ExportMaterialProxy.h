// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialDomain.h"
#include "MaterialShared.h"
#include "MaterialCompiler.h"
#include "TextureCompiler.h"
#include "Materials/MaterialParameterCollection.h"

#include "Engine/TextureLODSettings.h"
#include "Engine/Texture2D.h"
#include "Engine/Texture.h"
#include "Engine/TextureCube.h"
#include "Engine/Texture2DArray.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialRenderProxy.h"
#include "SceneTypes.h"
#include "Materials/Material.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/MaterialExpressionCustomOutput.h"

#include "RenderUtils.h"

struct FExportMaterialCompiler : public FProxyMaterialCompiler
{
	FExportMaterialCompiler(FMaterialCompiler* InCompiler) :
		FProxyMaterialCompiler(InCompiler)
	{}

	// gets value stored by SetMaterialProperty()
	virtual EShaderFrequency GetCurrentShaderFrequency() const override
	{
		return SF_Pixel;
	}

	virtual FMaterialShadingModelField GetMaterialShadingModels() const override
	{
		return Compiler->GetMaterialShadingModels();
	}

	virtual FMaterialShadingModelField GetCompiledShadingModels() const override
	{
		return Compiler->GetCompiledShadingModels();
	}

	virtual int32 WorldPosition(EWorldPositionIncludedOffsets WorldPositionIncludedOffsets) override
	{
#if WITH_EDITOR
		return MaterialBakingWorldPosition();
#else
		return Compiler->WorldPosition(WorldPositionIncludedOffsets);
#endif
	}

	virtual int32 DistanceCullFade() override
	{
		return Compiler->Constant(1.0f);
	}

	virtual int32 ParticleRelativeTime() override
	{
		return Compiler->Constant(0.0f);
	}

	virtual int32 ParticleMotionBlurFade() override
	{
		return Compiler->Constant(1.0f);
	}

	virtual int32 PixelNormalWS() override
	{
		// Current returning vertex normal since pixel normal will contain incorrect data (normal calculated from uv data used as vertex positions to render out the material)
		return Compiler->VertexNormal();
	}

	virtual int32 ParticleRandom() override
	{
		return Compiler->Constant(0.0f);
	}

	virtual int32 ParticleDirection() override
	{
		return Compiler->Constant3(0.0f, 0.0f, 0.0f);
	}

	virtual int32 ParticleSpeed() override
	{
		return Compiler->Constant(0.0f);
	}

	virtual int32 ParticleSize() override
	{
		return Compiler->Constant2(0.0f, 0.0f);
	}

	virtual int32 ParticleSpriteRotation() override
	{
		return Compiler->Constant2(0.0f, 0.0f);
	}

	virtual int32 CameraVector() override
	{
		// By returning vertex normal instead of a constant vector (like up), we ensure materials (with fresnel for example) are more correctly baked using custom mesh data.
		return Compiler->VertexNormal();
	}

	virtual int32 ReflectionAboutCustomWorldNormal(int32 CustomWorldNormal, int32 bNormalizeCustomWorldNormal) override
	{
		if (CustomWorldNormal == INDEX_NONE)
		{
			return INDEX_NONE;
	}

		int32 N = CustomWorldNormal;
		int32 C = CameraVector();

		if (bNormalizeCustomWorldNormal)
	{
			// N = N / sqrt(dot(N, N))
			N = Compiler->Div(N, Compiler->SquareRoot(Compiler->Dot(N, N)));
	}

		// return 2 * dot(N, C) * N - C
		return Compiler->Sub(Compiler->Mul(Compiler->Constant(2.0f), Compiler->Mul(Compiler->Dot(N, C), N)), C);
	}

	virtual int32 PreSkinnedPosition() override
	{
		return Compiler->PreSkinnedPosition();
	}

	virtual int32 PreSkinnedNormal() override
	{
		return Compiler->PreSkinnedNormal();
	}

	virtual int32 VertexInterpolator(uint32 InterpolatorIndex) override
	{
		return Compiler->VertexInterpolator(InterpolatorIndex);
	}

	virtual int32 ReflectionVector() override
	{
		// Because camera vector is identical to normal vector we can work out that reflection vector will also be the same
		return Compiler->VertexNormal();
	}

#if WITH_EDITOR
	virtual int32 MaterialBakingWorldPosition() override
	{
		// Depending on how the mesh data was retrieved, baking position may only be in local-space
		const int32 BakingPosition = Compiler->MaterialBakingWorldPosition();
		return Compiler->TransformPosition(MCB_Local, MCB_World, BakingPosition);
	}
#endif

	virtual int32 AccessCollectionParameter(UMaterialParameterCollection* ParameterCollection, int32 ParameterIndex, int32 ComponentIndex) override
	{
		if (!ParameterCollection || ParameterIndex == -1)
		{
			return INDEX_NONE;
		}

		// Collect names of all parameters
		TArray<FName> ParameterNames;
		ParameterCollection->GetParameterNames(ParameterNames, /*bVectorParameters=*/ false);
		int32 NumScalarParameters = ParameterNames.Num();
		ParameterCollection->GetParameterNames(ParameterNames, /*bVectorParameters=*/ true);

		// Find a parameter corresponding to ParameterIndex/ComponentIndex pair
		int32 Index;
		for (Index = 0; Index < ParameterNames.Num(); Index++)
		{
			FGuid ParameterId = ParameterCollection->GetParameterId(ParameterNames[Index]);
			int32 CheckParameterIndex, CheckComponentIndex;
			ParameterCollection->GetParameterIndex(ParameterId, CheckParameterIndex, CheckComponentIndex);
			if (CheckParameterIndex == ParameterIndex && CheckComponentIndex == ComponentIndex)
			{
				// Found
				break;
			}
		}
		if (Index >= ParameterNames.Num())
		{
			// Not found, should not happen
			return INDEX_NONE;
		}

		// Create code for parameter
		if (Index < NumScalarParameters)
		{
			const FCollectionScalarParameter* ScalarParameter = ParameterCollection->GetScalarParameterByName(ParameterNames[Index]);
			check(ScalarParameter);
			return Constant(ScalarParameter->DefaultValue);
		}
		else
		{
			const FCollectionVectorParameter* VectorParameter = ParameterCollection->GetVectorParameterByName(ParameterNames[Index]);
			check(VectorParameter);
			const FLinearColor& Color = VectorParameter->DefaultValue;
			return Constant4(Color.R, Color.G, Color.B, Color.A);
		}
	}

	virtual EMaterialCompilerType GetCompilerType() const override { return EMaterialCompilerType::MaterialProxy; }
};

class FExportMaterialProxy : public FEditorMaterialResourceBase, public FMaterialRenderProxy
{
public:
	FExportMaterialProxy(UMaterialInterface* InMaterialInterface, EMaterialProperty InPropertyToCompile, const FString& InCustomOutputToCompile = TEXT(""), bool bInSynchronousCompilation = true, bool bTangentSpaceNormal = false, EBlendMode ProxyBlendMode = BLEND_Opaque, bool bAllowPixelDepthOffset = true)
		: FEditorMaterialResourceBase()
		, FMaterialRenderProxy(GetPathNameSafe(InMaterialInterface->GetMaterial()))
		, MaterialInterface(InMaterialInterface)
		, PropertyToCompile(InPropertyToCompile)
		, CustomOutputToCompile(InCustomOutputToCompile)
		, bSynchronousCompilation(bInSynchronousCompilation)
		, bTangentSpaceNormal(bTangentSpaceNormal)
		, ProxyBlendMode(ProxyBlendMode)
	{
		SetQualityLevelProperties(GMaxRHIShaderPlatform);
		Material = InMaterialInterface->GetMaterial();
		ReferencedTextures = InMaterialInterface->GetDefaultTextures();
		ReferencedTextureCollections = InMaterialInterface->GetReferencedTextureCollections();

		const FMaterialResource* Resource = InMaterialInterface->GetMaterialResource(GMaxRHIShaderPlatform);

		FMaterialShaderMapId ResourceId;
		Resource->BuildShaderMapId(ResourceId, nullptr);

		{
			TArray<FShaderType*> ShaderTypes;
			TArray<FVertexFactoryType*> VFTypes;
			TArray<const FShaderPipelineType*> ShaderPipelineTypes;
			GetDependentShaderAndVFTypes(ResourceId.LayoutParams, ShaderTypes, ShaderPipelineTypes, VFTypes);

			// Overwrite the shader map Id's dependencies with ones that came from the FMaterial actually being compiled (this)
			// This is necessary as we change FMaterial attributes like GetShadingModels(), which factor into the ShouldCache functions that determine dependent shader types
			ResourceId.SetShaderDependencies(ShaderTypes, ShaderPipelineTypes, VFTypes, GMaxRHIShaderPlatform);
		}

		// Override with a special usage so we won't re-use the shader map used by the material for rendering
		switch (InPropertyToCompile)
		{
		case MP_BaseColor: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportBaseColor; break;
		case MP_Specular: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportSpecular; break;
		case MP_Normal: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportNormal; break;
		case MP_Tangent: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportTangent; break;
		case MP_Metallic: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportMetallic; break;
		case MP_Roughness: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportRoughness; break;
		case MP_Anisotropy: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportAnisotropy; break;
		case MP_AmbientOcclusion: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportAO; break;
		case MP_EmissiveColor: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportEmissive; break;
		case MP_Opacity: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportOpacity; break;
		case MP_Refraction: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportRefraction; break;
		case MP_OpacityMask: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportOpacityMask; break;
		case MP_SubsurfaceColor: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportSubSurfaceColor; break;
		case MP_ShadingModel: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportShadingModel; break;
		case MP_CustomData0: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportClearCoat; break;
		case MP_CustomData1: ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportClearCoatRoughness; break;
		case MP_CustomOutput:
			ResourceId.Usage = EMaterialShaderMapUsage::MaterialExportCustomOutput;
			ResourceId.UsageCustomOutput = InCustomOutputToCompile;
			break;
		default:
			ensureMsgf(false, TEXT("ExportMaterial has no usage for property %i.  Will likely reuse the normal rendering shader and crash later with a parameter mismatch"), (int32)InPropertyToCompile);
			break;
		};

		Usage = ResourceId.Usage;
		ResourceId.BaseMaterialId = Material->StateId;

		SetAllowPixelDepthOffset(bAllowPixelDepthOffset);
		
		CacheShaders(ResourceId);
	}

	/** This override is required otherwise the shaders aren't ready for use when the surface is rendered resulting in a blank image */
	virtual bool RequiresSynchronousCompilation() const override { return bSynchronousCompilation; };

	/**
	* Should the shader for this material with the given platform, shader type and vertex
	* factory type combination be compiled
	*
	* @param ShaderType	Which shader is being compiled
	* @param VertexFactory	Which vertex factory is being compiled (can be NULL)
	*
	* @return true if the shader should be compiled
	*/
	virtual bool ShouldCache(const FShaderType* ShaderType, const FVertexFactoryType* VertexFactoryType) const override
	{
		const bool bCorrectVertexFactory = VertexFactoryType == FindVertexFactoryType(FName(TEXT("FLocalVertexFactory"), FNAME_Find));
		const bool bPCPlatform = !IsConsolePlatform(GetShaderPlatform());
		const bool bCorrectFrequency = ShaderType->GetFrequency() == SF_Vertex || ShaderType->GetFrequency() == SF_Pixel;
		return bCorrectVertexFactory && bPCPlatform && bCorrectFrequency;
	}

	virtual void GetStaticParameterSet(FStaticParameterSet& OutSet) const override
	{
		if (const FMaterialResource* Resource = MaterialInterface->GetMaterialResource(GMaxRHIShaderPlatform))
		{
			Resource->GetStaticParameterSet(OutSet);
		}
	}

	////////////////
	// FMaterialRenderProxy interface.
	virtual const FMaterial* GetMaterialNoFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		if (GetRenderingThreadShaderMap())
		{
			return this;
		}
		return nullptr;
	}

	virtual const FMaterialRenderProxy* GetFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		return UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
	}

	virtual bool GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const override
	{
		return MaterialInterface->GetRenderProxy()->GetParameterValue(Type, ParameterInfo, OutValue, Context);
	}

	// Material properties.
	/** Entry point for compiling a specific material property.  This must call SetMaterialProperty. */
	virtual int32 CompilePropertyAndSetMaterialProperty(EMaterialProperty Property, FMaterialCompiler* Compiler, EShaderFrequency OverrideShaderFrequency, bool bUsePreviousFrameTime) const override
	{
		// needs to be called in this function!!
		Compiler->SetMaterialProperty(Property, OverrideShaderFrequency, bUsePreviousFrameTime);
		const int32 Ret = CompilePropertyAndSetMaterialPropertyWithoutCast(Property, Compiler);
		return Compiler->ForceCast(Ret, FMaterialAttributeDefinitionMap::GetValueType(Property), MFCF_ExactMatch | MFCF_ReplicateValue);
	}

	/** helper for CompilePropertyAndSetMaterialProperty() */
	int32 CompilePropertyAndSetMaterialPropertyWithoutCast(EMaterialProperty Property, FMaterialCompiler* Compiler) const
	{
		if (Property == MP_EmissiveColor)
		{
			FExportMaterialCompiler ProxyCompiler(Compiler);
			const uint32 ForceCast_Exact_Replicate = MFCF_ForceCast | MFCF_ExactMatch | MFCF_ReplicateValue;

			switch (PropertyToCompile)
			{
			case MP_EmissiveColor:
				Compiler->SetSubstrateMaterialExportType(SME_Emissive, ESubstrateMaterialExportContext::SMEC_Opaque, BLEND_Opaque);
				break;
			case MP_BaseColor:
				Compiler->SetSubstrateMaterialExportType(SME_BaseColor, ESubstrateMaterialExportContext::SMEC_Opaque, BLEND_Opaque);
				break;
			case MP_Specular:
				Compiler->SetSubstrateMaterialExportType(SME_Specular, ESubstrateMaterialExportContext::SMEC_Opaque, BLEND_Opaque);
				break;
			case MP_Roughness:
				Compiler->SetSubstrateMaterialExportType(SME_Roughness, ESubstrateMaterialExportContext::SMEC_Opaque, BLEND_Opaque);
				break;
			case MP_Anisotropy:
				Compiler->SetSubstrateMaterialExportType(SME_Anisotropy, ESubstrateMaterialExportContext::SMEC_Opaque, BLEND_Opaque);
				break;
			case MP_Metallic:
				Compiler->SetSubstrateMaterialExportType(SME_Metallic, ESubstrateMaterialExportContext::SMEC_Opaque, BLEND_Opaque);
				break;
			case MP_Opacity:
				Compiler->SetSubstrateMaterialExportType(SME_Opacity, ESubstrateMaterialExportContext::SMEC_Opaque, BLEND_Opaque);
				break;
			case MP_OpacityMask:
				Compiler->SetSubstrateMaterialExportType(SME_OpacityMask, ESubstrateMaterialExportContext::SMEC_Opaque, BLEND_Opaque);
				break;
			case MP_CustomData0:
				Compiler->SetSubstrateMaterialExportType(SME_CustomData0, ESubstrateMaterialExportContext::SMEC_Opaque, BLEND_Opaque);
				break;
			case MP_CustomData1:
				Compiler->SetSubstrateMaterialExportType(SME_CustomData1, ESubstrateMaterialExportContext::SMEC_Opaque, BLEND_Opaque);
				break;
			case MP_SubsurfaceColor:
				Compiler->SetSubstrateMaterialExportType(SME_SubsurfaceColor, ESubstrateMaterialExportContext::SMEC_Opaque, BLEND_Opaque);
				break;
			case MP_Normal:
				Compiler->SetSubstrateMaterialExportType(SME_Normal, ESubstrateMaterialExportContext::SMEC_Opaque, BLEND_Opaque);
				break;
			case MP_Tangent:
				Compiler->SetSubstrateMaterialExportType(SME_Tangent, ESubstrateMaterialExportContext::SMEC_Opaque, BLEND_Opaque);
				break;
			case MP_ShadingModel:
				Compiler->SetSubstrateMaterialExportType(SME_ShadingModel, ESubstrateMaterialExportContext::SMEC_Opaque, BLEND_Opaque);
				break;
			}


			switch (PropertyToCompile)
			{
			case MP_EmissiveColor:
			case MP_BaseColor:
			case MP_Specular:
			case MP_Roughness:
			case MP_Anisotropy:
			case MP_Metallic:
			case MP_AmbientOcclusion:
			case MP_Opacity:
			case MP_OpacityMask:
			case MP_CustomData0:
			case MP_CustomData1:
			case MP_SubsurfaceColor:
				return MaterialInterface->CompileProperty(&ProxyCompiler, PropertyToCompile, ForceCast_Exact_Replicate);
			case MP_Normal:
			case MP_Tangent:
				return CompileNormalEncoding(
					Compiler,
					CompileNormalTransform(&ProxyCompiler, MaterialInterface->CompileProperty(&ProxyCompiler, PropertyToCompile, ForceCast_Exact_Replicate)));
			case MP_Refraction:
				// Only index of refraction can be supported because other methods don't have values within a suitable range for encoding into 8-bit baked textures
				if (Material->RefractionMethod == RM_IndexOfRefraction)
				{
					return CompileRefractionEncoding(
						Compiler,
						MaterialInterface->CompileProperty(&ProxyCompiler, MP_Refraction, ForceCast_Exact_Replicate));
				}
				break;
			case MP_ShadingModel:
				return CompileShadingModelEncoding(Compiler, MaterialInterface->CompileProperty(&ProxyCompiler, MP_ShadingModel));
			case MP_CustomOutput:
				if (const FMaterialCustomOutputAttributeDefintion* CustomAttribute = FMaterialAttributeDefinitionMap::GetCustomAttribute(CustomOutputToCompile))
				{
					constexpr int32 InputIndex = 0; // Assume input index is always 0, which it is for all custom outputs that are registered as material attributes
					return CompileInputForCustomOutput(&ProxyCompiler, CustomAttribute, InputIndex, ForceCast_Exact_Replicate);
				}
				break;
			case MP_FrontMaterial:
				if (Substrate::IsSubstrateEnabled())
				{
					// When using Substrate, material property always compile from material. 
					// We cannot use rediction so instead we instruct the compiler the type of data export we are looking for.
					return MaterialInterface->CompileProperty(&ProxyCompiler, MP_FrontMaterial);
				}
				else
				{
					return ProxyCompiler.SubstrateCreateAndRegisterNullMaterial();
				}
			default:
				return Compiler->Constant(1.0f);
			}

			return Compiler->Constant(0.0f);
		}
		else if (Property == MP_WorldPositionOffset || Property == MP_Displacement)
		{
			//This property MUST return 0 as a default or during the process of rendering textures out for lightmass to use, pixels will be off by 1.
			return Compiler->Constant3(0.0f, 0.0f, 0.0f);
		}
		else if (Property >= MP_CustomizedUVs0 && Property <= MP_CustomizedUVs7)
		{
			// Pass through customized UVs
			return MaterialInterface->CompileProperty(Compiler, Property);
		}
		else if (Property == MP_OpacityMask)
		{
			return MaterialInterface->CompileProperty(Compiler, MP_OpacityMask);
		}
		else if (Property == MP_ShadingModel)
		{
			return MaterialInterface->CompileProperty(Compiler, MP_ShadingModel);
		}
		else if (Property == MP_SurfaceThickness)
		{
			return MaterialInterface->CompileProperty(Compiler, MP_SurfaceThickness);
		}
		else if (Property == MP_FrontMaterial)
		{
			if (Substrate::IsSubstrateEnabled())
			{
				return MaterialInterface->CompileProperty(Compiler, MP_FrontMaterial);
			}
			else
			{
				return Compiler->SubstrateCreateAndRegisterNullMaterial();
			}
		}
		else
		{
			if (Substrate::IsSubstrateEnabled() && Property == MP_Roughness)
			{
				// In this case return non 0.5 to not trigger the fully rough path from the Translator.
				// This is because we do not only compile EmissiveColor only anymore since we use the specified shading model (see GetShadingModels below).
				return Compiler->Constant(0.5f);
			}
			// Return the proper default value for each property so the translator doesn't
			// falsely detect usage of features like WPO, anisotropy, etc.
			return FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, Property);
		}
	}

	/**
	 * Gets the shader map usage of the material, which will be included in the DDC key.
	 * This mechanism allows derived material classes to create different DDC keys with the same base material.
	 */
	virtual EMaterialShaderMapUsage::Type GetShaderMapUsage() const override { return Usage; }

	virtual FString GetMaterialUsageDescription() const override
	{
		return FString::Printf(TEXT("MaterialBaking_%s"), MaterialInterface ? *MaterialInterface->GetName() : TEXT("NULL"));
	}
	virtual EMaterialDomain GetMaterialDomain() const override
	{
		// Because the baking module applies the material to a plane (or mesh),
		// it needs to be a surface material.
		return MD_Surface;
	}
	virtual bool IsTangentSpaceNormal() const override
	{
		if (const FMaterialResource* Resource = MaterialInterface->GetMaterialResource(GMaxRHIShaderPlatform))
		{
			return Resource->IsTangentSpaceNormal();
		}
		return false;
	}
	virtual bool IsTwoSided() const  override
	{
		if (MaterialInterface)
		{
			return MaterialInterface->IsTwoSided();
		}
		return false;
	}
	virtual bool IsThinSurface() const  override
	{
		if (MaterialInterface)
		{
			return MaterialInterface->IsThinSurface();
		}
		return false;
	}
	virtual bool IsDitheredLODTransition() const  override
	{
		if (MaterialInterface)
		{
			return MaterialInterface->IsDitheredLODTransition();
		}
		return false;
	}
	virtual bool IsLightFunction() const override
	{
		if (Material)
		{
			return (Material->MaterialDomain == MD_LightFunction);
		}
		return false;
	}
	virtual bool IsDeferredDecal() const override
	{
		// Decals are tricky. Since they mix with the underlying material
		// and can't be applied to meshes, they can't really be baked 1:1.
		// Instead we'll just bake them as surface materials.
		return false;
	}
	virtual bool IsUIMaterial() const override
	{
		return Material && Material->MaterialDomain == MD_UI;
	}
	virtual bool IsVolumetricPrimitive() const override
	{
		return Material && Material->MaterialDomain == MD_Volume;
	}
	virtual bool IsSpecialEngineMaterial() const override
	{
		if (Material)
		{
			return (Material->bUsedAsSpecialEngineMaterial == 1);
		}
		return false;
	}
	virtual bool IsWireframe() const override
	{
		if (Material)
		{
			return (Material->Wireframe == 1);
		}
		return false;
	}
	virtual bool IsMasked() const override { return ProxyBlendMode == BLEND_Masked; }
	virtual enum EBlendMode GetBlendMode() const override { return ProxyBlendMode; }
	virtual enum ERefractionMode GetRefractionMode() const override { return Material ? (ERefractionMode)Material->RefractionMethod : RM_None; }
	virtual bool GetRootNodeOverridesDefaultRefraction()const override { return false; }

	virtual FMaterialShadingModelField GetShadingModels() const override
	{
		// Substrate needs the real material shading model since the expressions access GetMaterialShadingModels() through the compiler to generate the substrate operators, 
		// and we do not want unlit materials when it is a Slab or a ShadingModel node.
		return Substrate::IsSubstrateEnabled() ? Material->GetShadingModels() : MSM_DefaultLit;
	}

	virtual bool IsShadingModelFromMaterialExpression() const override { return false; }
	virtual float GetOpacityMaskClipValue() const override { return 0.5f; }
	virtual bool GetCastDynamicShadowAsMasked() const override { return false; }
	virtual FString GetFriendlyName() const override { return FString::Printf(TEXT("FExportMaterialRenderer %s"), MaterialInterface ? *MaterialInterface->GetName() : TEXT("NULL")); }
	/**
	* Should shaders compiled for this material be saved to disk?
	*/
	virtual bool IsPersistent() const override { return true; }

	virtual FGuid GetMaterialId() const override
	{
		// Reuse the base material's Id
		// Normally this would cause a bug as the shader map would try to be shared by both, 
		// But FExportMaterialProxy::GetShaderMapUsage() allows this to work
		return Material->StateId;
	}

	virtual UMaterialInterface* GetMaterialInterface() const override
	{
		return MaterialInterface;
	}

	friend FArchive& operator<< (FArchive& Ar, FExportMaterialProxy& V)
	{
		return Ar << V.MaterialInterface;
	}

	virtual bool IsUsedWithStaticLighting() const override
	{
		return true; 
	}

	virtual void GatherExpressionsForCustomInterpolators(TArray<UMaterialExpression*>& OutExpressions) const override
	{
		if(Material)
		{
			Material->GetAllExpressionsForCustomInterpolators(OutExpressions);
		}
	}

	virtual bool CheckInValidStateForCompilation(FMaterialCompiler* Compiler) const override
	{
		return Material && Material->CheckInValidStateForCompilation(Compiler);
	}

private:
	int32 CompileInputForCustomOutput(FMaterialCompiler* Compiler, const FMaterialCustomOutputAttributeDefintion* CustomAttribute, int32 InputIndex, uint32 ForceCastFlags) const
	{
		check(CustomAttribute);

		UMaterialExpressionCustomOutput* Expression = GetCustomOutputExpression(CustomAttribute->FunctionName);
		FExpressionInput* ExpressionInput = Expression ? Expression->GetInput(InputIndex) : nullptr;
		int32 Result;

		if (ExpressionInput)
		{
			Result = ExpressionInput->Compile(Compiler);
		}
		else
		{
			Result = CustomAttribute->CompileDefaultValue(Compiler);
		}

		if (CustomOutputToCompile == TEXT("ClearCoatBottomNormal"))
		{
			Result = CompileNormalEncoding(Compiler, CompileNormalTransform(Compiler, Result));
		}

		if (ForceCastFlags & MFCF_ForceCast)
		{
			Result = Compiler->ForceCast(Result, CustomAttribute->ValueType, ForceCastFlags);
		}

		return Result;
	}

	UMaterialExpressionCustomOutput* GetCustomOutputExpression(const FString& FunctionName) const
	{
		for (UMaterialExpression* Expression : Material->GetExpressions())
		{
			UMaterialExpressionCustomOutput* CustomOutputExpression = Cast<UMaterialExpressionCustomOutput>(Expression);
			if (CustomOutputExpression && CustomOutputExpression->GetFunctionName() == FunctionName)
			{
				return CustomOutputExpression;
			}
		}

		return nullptr;
	}

	int32 CompileNormalTransform(FMaterialCompiler* Compiler, int32 NormalInput) const
	{
		return bTangentSpaceNormal && !Material->bTangentSpaceNormal
			? Compiler->TransformVector(MCB_World, MCB_Tangent, NormalInput) : NormalInput;
	}

	static int32 CompileNormalEncoding(FMaterialCompiler* Compiler, int32 NormalInput)
	{
		return Compiler->Add(
			Compiler->Mul(NormalInput, Compiler->Constant(0.5f)), // [-1,1] * 0.5
			Compiler->Constant(0.5f)); // [-0.5,0.5] + 0.5
	}

	static int32 CompileRefractionEncoding(FMaterialCompiler* Compiler, int32 RefractionInput)
	{
		// [1,Infinity] -> [1,0]
		return Compiler->Div(
			Compiler->Constant(1.0f),
			Compiler->Max(Compiler->Constant(1.0f), RefractionInput));
	}

	static int32 CompileShadingModelEncoding(FMaterialCompiler* Compiler, int32 ShadingModelInput)
	{
		// [0,MSM_NUM] -> [0,1]
		return Compiler->Div(
			Compiler->CastShadingModelToFloat(ShadingModelInput),
			Compiler->Constant(MSM_NUM));
	}

private:
	/** The material interface for this proxy */
	UMaterialInterface* MaterialInterface;
	UMaterial* Material;
	/** The property to compile for rendering the sample */
	EMaterialProperty PropertyToCompile;
	/** Stores which exported attribute this proxy is compiling for. */
	EMaterialShaderMapUsage::Type Usage;
	/** The name of the specific custom output to compile for rendering the sample. Only used if PropertyToCompile is MP_CustomOutput */
	FString CustomOutputToCompile;
	bool bSynchronousCompilation;

public:
	/** Whether to transform normals from world-space to tangent-space (does nothing if material already uses tangent-space normals) */
	bool bTangentSpaceNormal;
	/** The blend mode used when baking the proxy material */
	EBlendMode ProxyBlendMode;
};
