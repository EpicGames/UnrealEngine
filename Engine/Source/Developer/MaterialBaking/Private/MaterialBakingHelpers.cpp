// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialBakingHelpers.h"
#include "Math/Color.h"
#include "Async/ParallelFor.h"

namespace FMaterialBakingHelpersImpl
{
	static FColor BoxBlurSample(FColor* InBMP, int32 X, int32 Y, int32 InImageWidth, int32 InImageHeight, uint32 BackgroundDWColor)
	{
		int32 PixelsSampled = 0;
		int32 CombinedColorR = 0;
		int32 CombinedColorG = 0;
		int32 CombinedColorB = 0;
		int32 CombinedColorA = 0;

		const int32 BaseIndex = (Y * InImageWidth) + X;

		int32 SampleIndex = BaseIndex - (InImageWidth + 1);
		for (int32 YI = 0; YI < 3; ++YI)
		{
			for (int32 XI = 0; XI < 3; ++XI)
			{
				const FColor SampledColor = InBMP[SampleIndex++];
				// Check if the pixel is a rendered one (not clear color)
				if (SampledColor.DWColor() != BackgroundDWColor)
				{
					CombinedColorR += SampledColor.R;
					CombinedColorG += SampledColor.G;
					CombinedColorB += SampledColor.B;
					CombinedColorA += SampledColor.A;
					++PixelsSampled;
				}
			}
			SampleIndex += InImageWidth - 3;
		}

		if (PixelsSampled == 0)
		{
			return FColor(BackgroundDWColor);
		}

		return FColor(CombinedColorR / PixelsSampled, CombinedColorG / PixelsSampled, CombinedColorB / PixelsSampled, CombinedColorA / PixelsSampled);
	}

	static bool HasBorderingPixel(FColor* InBMP, int32 X, int32 Y, int32 InImageWidth, int32 InImageHeight, uint32 BackgroundDWColor)
	{
		const int32 BaseIndex = Y*InImageWidth + X;

		int32 SampleIndex = BaseIndex - (InImageWidth + 1);
		for (int32 YI = 0; YI < 3; ++YI)
		{
			for (int32 XI = 0; XI < 3; ++XI)
			{
				if (InBMP[SampleIndex++].DWColor() != BackgroundDWColor)
				{
					return true;
				}
			}
			SampleIndex += InImageWidth - 3;
		}
		return false;
	}

	static void PerformShrinking(TArray<FColor>& InOutPixels, int32& ImageWidth, int32& ImageHeight, uint32 BackgroundDWColor)
	{
		FColor* Current = InOutPixels.GetData();

		// We can hopefully skip the entire smearing process if there is a single
		// non-background color in the entire array since smearing that completely
		// would lead to a monochome output.
		uint32 SingleColor = BackgroundDWColor;
		bool   bHasSingleColor = true;
		for (int32 Index = 0, Num = InOutPixels.Num(); Index < Num; ++Index)
		{
			const uint32 Color = Current[Index].DWColor();
			if (Color != BackgroundDWColor)
			{
				if (SingleColor == BackgroundDWColor)
				{
					// This is the first time we stumble on a color other than background color, keep it.
					SingleColor = Color;
				}
				// Compare with the known non-background color
				else if (SingleColor != Color)
				{
					bHasSingleColor = false;
					break;
				}
			}
		}

		// Shrink to a single texel
		if (bHasSingleColor)
		{
			InOutPixels.SetNum(1);
			ImageWidth = ImageHeight = 1;
			Current = InOutPixels.GetData();

			InOutPixels[0] = FColor(SingleColor);
		}
	}

	static void PerformUVBorderSmear(TArray<FColor>& InOutPixels, int32& ImageWidth, int32& ImageHeight, int32 MaxIterations, uint32 BackgroundDWColor)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialBakingHelpers::PerformUVBorderSmear)

		check(InOutPixels.Num() == ImageWidth*ImageHeight);

		// No smearing needed for a single pixel
		if (ImageWidth * ImageHeight == 1)
		{
			return;
		}

		const int32 PaddedImageWidth = ImageWidth + 2;
		const int32 PaddedImageHeight = ImageHeight + 2;

		TArray<uint32> RowsSwap1;
		TArray<uint32> RowsSwap2;
		TArray<uint32>* CurrentRowsLeft = &RowsSwap1;
		TArray<uint32>* NextRowsLeft = &RowsSwap2;

		//
		// Create a copy of the image with a 1 pixel background color border around the edge.
		// This avoids needing to bounds check in inner loops.
		//

		TArray<FColor> ScratchBuffer;
		ScratchBuffer.SetNumUninitialized(PaddedImageWidth*PaddedImageHeight);
		FColor* Scratch = ScratchBuffer.GetData();

		// Set top and bottom bordering pixels to background color
		for (int32 X = 0; X < PaddedImageWidth; ++X)
		{
			Scratch[X] = FColor(BackgroundDWColor);
			Scratch[(PaddedImageHeight - 1)*PaddedImageWidth + X] = FColor(BackgroundDWColor);
		}

		FColor* Current = InOutPixels.GetData();

		// Set left and right border pixels to background color and copy image data into rows
		for (int32 Y = 1; Y <= ImageHeight; ++Y)
		{
			const int32 YOffset = Y * PaddedImageWidth;
			Scratch[YOffset] = FColor(BackgroundDWColor);
			FMemory::Memcpy(&Scratch[YOffset + 1], &Current[(Y - 1)*ImageWidth], ImageWidth * sizeof(FColor));
			Scratch[YOffset + ImageWidth + 1] = FColor(BackgroundDWColor);
		}

		//
		// Find our initial workset of all rows that have background colored pixels bordering non-background pixels.
		// Also find all rows that have zero background colored pixels - these will never be added to the workset.
		//

		TArray<bool> RowCompleted;
		RowCompleted.SetNumZeroed(PaddedImageHeight);

		// Set border rows to start completed
		RowCompleted[0] = true;
		RowCompleted[PaddedImageHeight - 1] = true;

		bool bHasAnyData = false;
		for (int32 Y = 1; Y <= ImageHeight; ++Y)
		{
			bool bHasBackground = false;
			bool bBordersNonBackground = false;
			const int32 YOffset = Y * PaddedImageWidth;
			for (int32 X = 1; X <= ImageWidth; ++X)
			{
				if (Scratch[YOffset + X].DWColor() == BackgroundDWColor)
				{
					bHasBackground = true;
					if (HasBorderingPixel(Scratch, X, Y, PaddedImageWidth, PaddedImageHeight, BackgroundDWColor))
					{
						bBordersNonBackground = true;
						break;
					}
				}
			}

			if (!bHasBackground)
			{
				bHasAnyData = true;
				RowCompleted[Y] = true;
			}
			else if (bBordersNonBackground)
			{
				bHasAnyData = true;
				CurrentRowsLeft->Add(Y);
			}
		}

		// early-out if no valid pixels were found.
		if (!bHasAnyData)
		{
			FMemory::Memzero(InOutPixels.GetData(), ImageWidth * ImageHeight * sizeof(FColor));
			return;
		}

		// Early out if all pixels were valid.
		if (CurrentRowsLeft->Num() == 0)
		{
			return;
		}

		TArray<uint32> RowRemainingPixels;
		RowRemainingPixels.SetNumZeroed(PaddedImageHeight);

		const int32 MaxThreads = FPlatformProcess::SupportsMultithreading() ? FPlatformMisc::NumberOfCoresIncludingHyperthreads() : 1;

		//
		// Iteratively smear until all rows are filled.
		//

		int32 LoopCount = 0;
		while (CurrentRowsLeft->Num() && (MaxIterations <= 0 || LoopCount <= MaxIterations))
		{
			const int32 NumThreads = FMath::Min(CurrentRowsLeft->Num(), MaxThreads);
			const int32 LinesPerThread = FMath::CeilToInt((float)CurrentRowsLeft->Num() / (float)NumThreads);

			// split up rows that still have background colored pixels amongst threads
			ParallelFor(NumThreads, [ImageWidth, PaddedImageWidth, PaddedImageHeight, Current, Scratch, CurrentRowsLeft, LinesPerThread, &RowRemainingPixels, BackgroundDWColor](int32 ThreadIndex)
			{
				const int32 StartY = ThreadIndex*LinesPerThread;
				const int32 EndY = FMath::Min((ThreadIndex + 1) * LinesPerThread, CurrentRowsLeft->Num());

				for (int32 Index = StartY; Index < EndY; Index++)
				{
					const int32 Y = (*CurrentRowsLeft)[Index];
					RowRemainingPixels[Index] = 0;

					int32 PixelIndex = (Y - 1) * ImageWidth;
					for (int32 X = 1; X <= ImageWidth; X++)
					{
						FColor& Color = Current[PixelIndex++];
						if (Color.DWColor() == BackgroundDWColor)
						{
							const FColor SampledColor = BoxBlurSample(Scratch, X, Y, PaddedImageWidth, PaddedImageHeight, BackgroundDWColor);
							// If it's a valid pixel
							if (SampledColor.DWColor() != BackgroundDWColor)
							{
								Color = SampledColor;
							}
							else
							{
								RowRemainingPixels[Index]++;
							}
						}
					}
				}
			});

			// Mark all completed rows and copy data between buffers
			const int32 RowMemorySize = ImageWidth * sizeof(FColor);
			for (int32 RowIndex = 0; RowIndex < CurrentRowsLeft->Num(); ++RowIndex)
			{
				const int32 Y = (*CurrentRowsLeft)[RowIndex];
				if (!RowRemainingPixels[RowIndex])
				{
					RowCompleted[Y] = true;
				}
				FMemory::Memcpy(&Scratch[Y * PaddedImageWidth + 1], &Current[(Y - 1) * ImageWidth], RowMemorySize);
			}

			// Find the next set of rows to process.
			// This will be our current rows that haven't completed, plus any bordering rows that haven't been marked completed.
			NextRowsLeft->SetNum(0, EAllowShrinking::No);
			{
				int32 PreviousY = -1;
				for (int32 RowIndex = 0; RowIndex < CurrentRowsLeft->Num(); ++RowIndex)
				{
					const int32 Y = (*CurrentRowsLeft)[RowIndex];
					if (!RowCompleted[Y - 1] && PreviousY < Y - 1)
					{
						PreviousY = Y - 1;
						NextRowsLeft->Add(Y - 1);
					}
					if (PreviousY < Y && !RowCompleted[Y] && RowRemainingPixels[RowIndex])
					{
						PreviousY = Y;
						NextRowsLeft->Add(Y);
					}
					if (!RowCompleted[Y + 1])
					{
						PreviousY = Y + 1;
						NextRowsLeft->Add(Y + 1);
					}
				}
			}

			// Swap rows left buffers
			{
				TArray<uint32>* Temp = NextRowsLeft;
				NextRowsLeft = CurrentRowsLeft;
				CurrentRowsLeft = Temp;
			}

			LoopCount++;
		}

		// If we finished before replacing all pixels, replace the remaining pixels with black.
		for (int32 Index = 0; Index < CurrentRowsLeft->Num(); Index++)
		{
			const int32 Y = (*CurrentRowsLeft)[Index];

			int32 PixelIndex = (Y - 1) * ImageWidth;
			for (int32 X = 1; X <= ImageWidth; X++)
			{
				FColor& Color = Current[PixelIndex++];
				if (Color.DWColor() == BackgroundDWColor)
				{
					Color = FColor::Black;
				}
			}
		}
	}
}

void FMaterialBakingHelpers::PerformUVBorderSmear(TArray<FColor>& InOutPixels, int32 ImageWidth, int32 ImageHeight, int32 MaxIterations, FColor BackgroundColor)
{
	FMaterialBakingHelpersImpl::PerformUVBorderSmear(InOutPixels, ImageWidth, ImageHeight, MaxIterations, BackgroundColor.DWColor());
}

void FMaterialBakingHelpers::PerformShrinking(TArray<FColor>& InOutPixels, int32& InOutImageWidth, int32& InOutImageHeight, FColor BackgroundColor)
{
	FMaterialBakingHelpersImpl::PerformShrinking(InOutPixels, InOutImageWidth, InOutImageHeight, BackgroundColor.DWColor());
}

void FMaterialBakingHelpers::PerformUVBorderSmearAndShrink(TArray<FColor>& InOutPixels, int32& InOutImageWidth, int32& InOutImageHeight, FColor BackgroundColor)
{
	const uint32 BackgroundDWColor = BackgroundColor.DWColor();
	FMaterialBakingHelpersImpl::PerformShrinking(InOutPixels, InOutImageWidth, InOutImageHeight, BackgroundDWColor);

	const int32 DefaultMaxIteration = -1; // Perform smearing over the whole image
	FMaterialBakingHelpersImpl::PerformUVBorderSmear(InOutPixels, InOutImageWidth, InOutImageHeight, DefaultMaxIteration, BackgroundDWColor);
}