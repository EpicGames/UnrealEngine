// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialBakingModule.h"
#include "ContentStreaming.h"
#include "MaterialRenderItem.h"
#include "Engine/TextureRenderTarget2D.h"
#include "ExportMaterialProxy.h"
#include "Interfaces/IMainFrameModule.h"
#include "MaterialOptionsWindow.h"
#include "MaterialOptions.h"
#include "PropertyEditorModule.h"
#include "MaterialOptionsCustomization.h"
#include "UObject/UObjectGlobals.h"
#include "MaterialBakingStructures.h"
#include "Framework/Application/SlateApplication.h"
#include "MaterialBakingHelpers.h"
#include "Async/Async.h"
#include "Async/ParallelFor.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceConstant.h"
#include "MaterialEditor/MaterialEditorInstanceConstant.h"
#include "RenderingThread.h"
#include "RHISurfaceDataConversion.h"
#include "SceneView.h"
#include "Serialization/ArchiveCrc32.h"
#include "Misc/ScopedSlowTask.h"
#include "MeshDescription.h"
#include "TextureCompiler.h"
#include "TextureResource.h"
#include "RenderCaptureInterface.h"
#if WITH_EDITOR
#include "Misc/FileHelper.h"
#endif
#include "Templates/Greater.h"

IMPLEMENT_MODULE(FMaterialBakingModule, MaterialBaking);

DEFINE_LOG_CATEGORY_STATIC(LogMaterialBaking, Log, All);

#define LOCTEXT_NAMESPACE "MaterialBakingModule"

/** Cvars for advanced features */
static TAutoConsoleVariable<int32> CVarUseMaterialProxyCaching(
	TEXT("MaterialBaking.UseMaterialProxyCaching"),
	1,
	TEXT("Determines whether or not Material Proxies should be cached to speed up material baking.\n")
	TEXT("0: Turned Off\n")
	TEXT("1: Turned On"),	
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarSaveIntermediateTextures(
	TEXT("MaterialBaking.SaveIntermediateTextures"),
	0,
	TEXT("Determines whether or not to save out intermediate BMP images for each flattened material property.\n")
	TEXT("0: Turned Off\n")
	TEXT("1: Turned On"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarMaterialBakingRDOCCapture(
	TEXT("MaterialBaking.RenderDocCapture"),
	0,
	TEXT("Determines whether or not to trigger a RenderDoc capture.\n")
	TEXT("0: Turned Off\n")
	TEXT("1: Turned On"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarMaterialBakingVTWarmupFrames(
	TEXT("MaterialBaking.VTWarmupFrames"),
	5,
	TEXT("Number of frames to render for virtual texture warmup when material baking."));

static TAutoConsoleVariable<bool> CVarMaterialBakingForceDisableEmissiveScaling(
	TEXT("MaterialBaking.ForceDisableEmissiveScaling"),
	false,
	TEXT("If set to true, values stored in the emissive textures will be clamped to the [0, 1] range rather than being normalized and scaled back using the EmissiveScale material static parameter."));

namespace
{
	// Custom dynamic mesh allocator specifically tailored for Material Baking.
	// This will always reuse the same couple buffers, so searching linearly is not a problem.
	class FMaterialBakingDynamicMeshBufferAllocator : public FDynamicMeshBufferAllocator
	{
		// This must be smaller than the large allocation blocks on Windows 10 which is currently ~508K.
		// Large allocations uses VirtualAlloc directly without any kind of buffering before
		// releasing pages to the kernel, so it causes lots of soft page fault when
		// memory is first initialized.
		const uint32 SmallestPooledBufferSize = 256*1024;

		TArray<FBufferRHIRef>  IndexBuffers;
		TArray<FBufferRHIRef> VertexBuffers;

		template <typename RefType>
		RefType GetSmallestFit(uint32 SizeInBytes, TArray<RefType>& Array)
		{
			uint32 SmallestFitIndex = UINT32_MAX;
			uint32 SmallestFitSize  = UINT32_MAX;
			for (int32 Index = 0; Index < Array.Num(); ++Index)
			{
				uint32 Size = Array[Index]->GetSize();
				if (Size >= SizeInBytes && (SmallestFitIndex == UINT32_MAX || Size < SmallestFitSize))
				{
					SmallestFitIndex = Index;
					SmallestFitSize  = Size;
				}
			}

			RefType Ref;
			// Do not reuse the smallest fit if it's a lot bigger than what we requested
			if (SmallestFitIndex != UINT32_MAX && SmallestFitSize < SizeInBytes*2)
			{
				Ref = Array[SmallestFitIndex];
				Array.RemoveAtSwap(SmallestFitIndex);
			}

			return Ref;
		}

		virtual FBufferRHIRef AllocIndexBuffer(FRHICommandListBase& RHICmdList, uint32 NumElements) override
		{
			uint32 BufferSize = GetIndexBufferSize(NumElements);
			if (BufferSize > SmallestPooledBufferSize)
			{
				FBufferRHIRef Ref = GetSmallestFit(GetIndexBufferSize(NumElements), IndexBuffers);
				if (Ref.IsValid())
				{
					return Ref;
				}
			}

			return FDynamicMeshBufferAllocator::AllocIndexBuffer(RHICmdList, NumElements);
		}

		virtual void ReleaseIndexBuffer(FBufferRHIRef& IndexBufferRHI) override
		{
			if (IndexBufferRHI->GetSize() > SmallestPooledBufferSize)
			{
				IndexBuffers.Add(MoveTemp(IndexBufferRHI));
			}

			IndexBufferRHI = nullptr;
		}

		virtual FBufferRHIRef AllocVertexBuffer(FRHICommandListBase& RHICmdList, uint32 Stride, uint32 NumElements) override
		{
			uint32 BufferSize = GetVertexBufferSize(Stride, NumElements);
			if (BufferSize > SmallestPooledBufferSize)
			{
				FBufferRHIRef Ref = GetSmallestFit(BufferSize, VertexBuffers);
				if (Ref.IsValid())
				{
					return Ref;
				}
			}

			return FDynamicMeshBufferAllocator::AllocVertexBuffer(RHICmdList, Stride, NumElements);
		}

		virtual void ReleaseVertexBuffer(FBufferRHIRef& VertexBufferRHI) override
		{
			if (VertexBufferRHI->GetSize() > SmallestPooledBufferSize)
			{
				VertexBuffers.Add(MoveTemp(VertexBufferRHI));
			}

			VertexBufferRHI = nullptr;
		}
	};

	class FStagingBufferPool
	{
	public:
		FTextureRHIRef CreateStagingBuffer_RenderThread(FRHICommandListImmediate& RHICmdList, int32 Width, int32 Height, EPixelFormat Format, bool bIsSRGB)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(CreateStagingBuffer_RenderThread)

			auto StagingBufferPredicate = 
				[Width, Height, Format, bIsSRGB](const FTextureRHIRef& Texture2DRHIRef)
				{
					return Texture2DRHIRef->GetSizeX() == Width && Texture2DRHIRef->GetSizeY() == Height && Texture2DRHIRef->GetFormat() == Format && bool(Texture2DRHIRef->GetFlags() & TexCreate_SRGB) == bIsSRGB;
				};

			// Process any staging buffers available for unmapping
			{
				TArray<FTextureRHIRef> ToUnmapLocal;
				{
					FScopeLock Lock(&ToUnmapLock);
					ToUnmapLocal = MoveTemp(ToUnmap);
				}

				for (int32 Index = 0, Num = ToUnmapLocal.Num(); Index < Num; ++Index)
				{
					RHICmdList.UnmapStagingSurface(ToUnmapLocal[Index]);
					Pool.Add(MoveTemp(ToUnmapLocal[Index]));
				}
			}

			// Find any pooled staging buffer with suitable properties.
			int32 Index = Pool.IndexOfByPredicate(StagingBufferPredicate);

			if (Index != -1)
			{
				FTextureRHIRef StagingBuffer = MoveTemp(Pool[Index]);
				Pool.RemoveAtSwap(Index);
				return StagingBuffer;
			}

			TRACE_CPUPROFILER_EVENT_SCOPE(RHICreateTexture2D)

			FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create2D(TEXT("FStagingBufferPool_StagingBuffer"), Width, Height, Format)
				.SetFlags(ETextureCreateFlags::CPUReadback);

			if (bIsSRGB)
			{
				Desc.AddFlags(ETextureCreateFlags::SRGB);
			}

			return RHICmdList.CreateTexture(Desc);
		}

		void ReleaseStagingBufferForUnmap_AnyThread(FTextureRHIRef& Texture2DRHIRef)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ReleaseStagingBufferForUnmap_AnyThread)
			FScopeLock Lock(&ToUnmapLock);
			ToUnmap.Emplace(MoveTemp(Texture2DRHIRef));
		}

		void Clear_RenderThread(FRHICommandListImmediate& RHICmdList)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Clear_RenderThread)
			for (FTextureRHIRef& StagingSurface : ToUnmap)
			{
				RHICmdList.UnmapStagingSurface(StagingSurface);
			}

			ToUnmap.Empty();
			Pool.Empty();
		}

		~FStagingBufferPool()
		{
			check(Pool.Num() == 0);
		}

	private:
		TArray<FTextureRHIRef> Pool;

		// Not contented enough to warrant the use of lockless structures.
		FCriticalSection       ToUnmapLock;
		TArray<FTextureRHIRef> ToUnmap;
	};

	struct FRenderItemKey
	{
		const FMeshData* RenderData;
		const FIntPoint  RenderSize;

		FRenderItemKey(const FMeshData* InRenderData, const FIntPoint& InRenderSize)
			: RenderData(InRenderData)
			, RenderSize(InRenderSize)
		{
		}

		bool operator == (const FRenderItemKey& Other) const
		{
			return RenderData == Other.RenderData &&
				RenderSize == Other.RenderSize;
		}
	};

	uint32 GetTypeHash(const FRenderItemKey& Key)
	{
		return HashCombine(GetTypeHash(Key.RenderData), GetTypeHash(Key.RenderSize));
	}
}

/** Helper for emissive color conversion to Output */
static void ProcessEmissiveOutput(const FFloat16Color* Color16, int32 Color16Pitch, const FIntPoint& OutputSize, TArray<FColor>& Output, float& EmissiveScale, const FColor& BackgroundColor);

void FMaterialBakingModule::StartupModule()
{
	bEmissiveHDR = false;

	bIsBakingMaterials = false;

	// Set which properties should enforce gamma correction
	SetLinearBake(true);

	// Set which pixel format should be used for the possible baked out material properties
	PerPropertyFormat.Add(MP_EmissiveColor, PF_FloatRGBA);
	PerPropertyFormat.Add(MP_Opacity, PF_B8G8R8A8);
	PerPropertyFormat.Add(MP_OpacityMask, PF_B8G8R8A8);
	PerPropertyFormat.Add(MP_BaseColor, PF_B8G8R8A8);
	PerPropertyFormat.Add(MP_Metallic, PF_B8G8R8A8);
	PerPropertyFormat.Add(MP_Specular, PF_B8G8R8A8);
	PerPropertyFormat.Add(MP_Roughness, PF_B8G8R8A8);
	PerPropertyFormat.Add(MP_Anisotropy, PF_B8G8R8A8);
	PerPropertyFormat.Add(MP_Refraction, PF_B8G8R8A8);
	PerPropertyFormat.Add(MP_Normal, PF_B8G8R8A8);
	PerPropertyFormat.Add(MP_Tangent, PF_B8G8R8A8);
	PerPropertyFormat.Add(MP_AmbientOcclusion, PF_B8G8R8A8);
	PerPropertyFormat.Add(MP_SubsurfaceColor, PF_B8G8R8A8);
	PerPropertyFormat.Add(MP_CustomData0, PF_B8G8R8A8);
	PerPropertyFormat.Add(MP_CustomData1, PF_B8G8R8A8);
	PerPropertyFormat.Add(MP_ShadingModel, PF_B8G8R8A8);
	PerPropertyFormat.Add(FMaterialPropertyEx::ClearCoatBottomNormal, PF_B8G8R8A8);
	PerPropertyFormat.Add(FMaterialPropertyEx::TransmittanceColor, PF_B8G8R8A8);

	// Register property customization
	FPropertyEditorModule& Module = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	Module.RegisterCustomPropertyTypeLayout(TEXT("PropertyEntry"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPropertyEntryCustomization::MakeInstance));
	
	// Register callback for modified objects
	FCoreUObjectDelegates::OnObjectModified.AddRaw(this, &FMaterialBakingModule::OnObjectModified);

	// Register callback on garbage collection
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddRaw(this, &FMaterialBakingModule::OnPreGarbageCollect);
}

void FMaterialBakingModule::ShutdownModule()
{
	// Unregister customization and callback
	FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if (PropertyEditorModule)
	{
		PropertyEditorModule->UnregisterCustomPropertyTypeLayout(TEXT("PropertyEntry"));
	}

	FCoreUObjectDelegates::OnObjectModified.RemoveAll(this);
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().RemoveAll(this);

	CleanupMaterialProxies();

	CleanupRenderTargets();
}

uint32 FMaterialBakingModule::GetCRC() const
{
	FArchiveCrc32 Ar;

	// Base key, changing this will force a rebuild of all HLODs that are relying on material baking.
	FString ModuleBaseKey = "4167B9A126CA47B3A6EAB520B40A66BB";
	Ar << ModuleBaseKey;

	bool bUseEmissiveHDR = bEmissiveHDR;
	Ar << bUseEmissiveHDR;

	uint8 ColorSpace = DefaultColorSpace;
	Ar << ColorSpace;

	int32 VTWarmupFrames = CVarMaterialBakingVTWarmupFrames.GetValueOnAnyThread();
	Ar << VTWarmupFrames;

	bool ForceDisableEmissiveScaling = CVarMaterialBakingForceDisableEmissiveScaling.GetValueOnAnyThread();
	Ar << ForceDisableEmissiveScaling;

	return Ar.GetCrc();
}

FMaterialDataEx ToMaterialDataEx(const FMaterialData& MaterialData)
{
	FMaterialDataEx MaterialDataEx;
	MaterialDataEx.Material = MaterialData.Material;
	MaterialDataEx.bPerformBorderSmear = MaterialData.bPerformBorderSmear;
	MaterialDataEx.bPerformShrinking = MaterialData.bPerformShrinking;
	MaterialDataEx.bTangentSpaceNormal = MaterialData.bTangentSpaceNormal;
	MaterialDataEx.BlendMode = MaterialData.BlendMode;
	MaterialDataEx.BackgroundColor = MaterialData.BackgroundColor;
	for (const TPair<EMaterialProperty, FIntPoint>& PropertySizePair : MaterialData.PropertySizes)
	{
		MaterialDataEx.PropertySizes.Add(PropertySizePair.Key, PropertySizePair.Value);
	}
	return MaterialDataEx;
}

FBakeOutput ToBakeOutput(FBakeOutputEx& BakeOutputEx)
{
	FBakeOutput BakeOutput;

	BakeOutput.EmissiveScale = BakeOutputEx.EmissiveScale;

	for (TPair<FMaterialPropertyEx, FIntPoint>& PropertySizePair : BakeOutputEx.PropertySizes)
	{
		BakeOutput.PropertySizes.Add(PropertySizePair.Key.Type, PropertySizePair.Value);
	}

	for (TPair<FMaterialPropertyEx, TArray<FColor>>& PropertyDataPair : BakeOutputEx.PropertyData)
	{
		BakeOutput.PropertyData.Add(PropertyDataPair.Key.Type, MoveTemp(PropertyDataPair.Value));
	}

	for (TPair<FMaterialPropertyEx, TArray<FFloat16Color>>& PropertyDataPair : BakeOutputEx.HDRPropertyData)
	{
		BakeOutput.HDRPropertyData.Add(PropertyDataPair.Key.Type, MoveTemp(PropertyDataPair.Value));
	}

	for (TPair<FMaterialPropertyEx, bool>& PropertyDataPair : BakeOutputEx.PropertyIsLinearColor)
	{
		BakeOutput.PropertyIsLinearColor.Add(PropertyDataPair.Key.Type, PropertyDataPair.Value);
	}

	return BakeOutput;
}

void FMaterialBakingModule::BakeMaterials(const TArray<FMaterialData*>& MaterialSettings, const TArray<FMeshData*>& MeshSettings, TArray<FBakeOutput>& Output)
{
	// Translate old material data to extended types
	TArray<FMaterialDataEx> MaterialDataExs;
	MaterialDataExs.Reserve(MaterialSettings.Num());
	for (const FMaterialData* MaterialData : MaterialSettings)
	{
		MaterialDataExs.Emplace(ToMaterialDataEx(*MaterialData));
	}

	// Build an array of pointers to the extended type
	TArray<FMaterialDataEx*> MaterialSettingsEx;
	MaterialSettingsEx.Reserve(MaterialDataExs.Num());
	for (FMaterialDataEx& MaterialDataEx : MaterialDataExs)
	{
		MaterialSettingsEx.Add(&MaterialDataEx);
	}

	TArray<FBakeOutputEx> BakeOutputExs;
	BakeMaterials(MaterialSettingsEx, MeshSettings, BakeOutputExs);

	// Translate extended bake output to old types
	Output.Reserve(BakeOutputExs.Num());
	for (FBakeOutputEx& BakeOutputEx : BakeOutputExs)
	{
		Output.Emplace(ToBakeOutput(BakeOutputEx));
	}
}

void FMaterialBakingModule::BakeMaterials(const TArray<FMaterialData*>& MaterialSettings, const TArray<FMeshData*>& MeshSettings, FBakeOutput& BakeOutput)
{
	// Translate old material data to extended types
	TArray<FMaterialDataEx> MaterialDataExs;
	MaterialDataExs.Reserve(MaterialSettings.Num());
	for (const FMaterialData* MaterialData : MaterialSettings)
	{
		MaterialDataExs.Emplace(ToMaterialDataEx(*MaterialData));
	}

	// Build an array of pointers to the extended type
	TArray<FMaterialDataEx*> MaterialSettingsEx;
	MaterialSettingsEx.Reserve(MaterialDataExs.Num());
	for (FMaterialDataEx& MaterialDataEx : MaterialDataExs)
	{
		MaterialSettingsEx.Add(&MaterialDataEx);
	}

	FBakeOutputEx BakeOutputEx;
	BakeMaterials(MaterialSettingsEx, MeshSettings, BakeOutputEx);

	// Translate extended bake output to old type
	BakeOutput = ToBakeOutput(BakeOutputEx);
}


class FMaterialBakingProcessor
{
public:
	FMaterialBakingProcessor(FMaterialBakingModule& InMaterialBakingModule, const TArray<FMaterialDataEx*>& InMaterialSettings, const TArray<FMeshData*>& InMeshSettings)
		: MaterialBakingModule(InMaterialBakingModule)
		, MaterialSettings(InMaterialSettings)
		, MeshSettings(InMeshSettings)
		, bSaveIntermediateTextures(CVarSaveIntermediateTextures.GetValueOnAnyThread() == 1)
		, bEmissiveHDR(InMaterialBakingModule.bEmissiveHDR)
	{
		checkf(MaterialSettings.Num() == MeshSettings.Num(), TEXT("Number of material settings does not match that of MeshSettings"));

		UE_LOGF(LogMaterialBaking, Verbose, "Performing material baking for %d materials", MaterialSettings.Num());
		for (int32 i = 0; i < MaterialSettings.Num(); i++)
		{
			if (MaterialSettings[i]->Material && MeshSettings[i]->MeshDescription)
			{
				UE_LOGF(LogMaterialBaking, Verbose, "    [%5d] Material: %-50ls Vertices: %8d    Triangles: %8d", i, *MaterialSettings[i]->Material->GetName(), MeshSettings[i]->MeshDescription->Vertices().Num(), MeshSettings[i]->MeshDescription->Triangles().Num());
			}
		}

		ComputeMeshProcessingOrder();

		NumMaterials = ProcessingOrder.Num();
	}

	virtual ~FMaterialBakingProcessor() = default;

	void BakeMaterials()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialBakingModule::BakeMaterials)

		// We reuse the pipeline depth to prepare render items in advance to avoid stalling the game thread
		LaunchAsyncPrepareRenderItems(PipelineDepth);

		// Create all material proxies right away to start compiling shaders asynchronously and avoid stalling the baking process as much as possible
		CreateMaterialProxies();

		// For each material
		for (int32 Index = 0; Index < NumMaterials; ++Index)
		{
			const int32 MaterialIndex = ProcessingOrder[Index];

			const FMaterialDataEx& CurrentMaterialSettings = *MaterialSettings[MaterialIndex];
			check(!CurrentMaterialSettings.PropertySizes.IsEmpty());

			const FMeshData* CurrentMeshSettings = MeshSettings[MaterialIndex];
			FBakeOutputEx& CurrentOutput = GetBakeOutput(MaterialIndex);

			TMap<FRenderItemKey, FMeshMaterialRenderItem*>* RenderItems = GetRenderItems(Index);
			check(RenderItems && !RenderItems->IsEmpty());

			// For each property
			for (const auto& [Property, Size] : CurrentMaterialSettings.PropertySizes)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*Property.ToString())

				FExportMaterialProxy* ExportMaterialProxy = MaterialBakingModule.CreateMaterialProxy(&CurrentMaterialSettings, Property);
				if (!ExportMaterialProxy->IsCompilationFinished())
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(WaitForMaterialProxyCompilation)
					ExportMaterialProxy->FinishCompilation();
				}

				UTextureRenderTarget2D* RenderTarget = GetRenderTarget(Property, Size, CurrentMaterialSettings);
				FMeshMaterialRenderItem* RenderItem = RenderItems->FindChecked(FRenderItemKey(CurrentMeshSettings, Size));

				BakeMaterialProperty(CurrentMaterialSettings, Property, RenderItem, RenderTarget, ExportMaterialProxy, CurrentOutput);
			}

			// Destroying Render Items
			// Must happen on the render thread to ensure they are not used anymore.
			ENQUEUE_RENDER_COMMAND(DestroyRenderItems)(
				[RenderItems](FRHICommandListImmediate& RHICmdList)
				{
					for (auto RenderItem : (*RenderItems))
					{
						delete RenderItem.Value;
					}

					delete RenderItems;
				}
			);
		}
	}

protected:
	void PrepareBakeOutput(FMaterialDataEx* InMaterialSettings, FBakeOutputEx& BakeOutput)
	{
		BakeOutput.PropertySizes = InMaterialSettings->PropertySizes;

		for (const auto& [Property, Size] : BakeOutput.PropertySizes)
		{
			BakeOutput.PropertyData.Add(Property);
			if (bEmissiveHDR && Property == MP_EmissiveColor)
			{
				BakeOutput.HDRPropertyData.Add(Property);
			}

			BakeOutput.PropertyIsLinearColor.Add(Property, MaterialBakingModule.IsLinearBake(Property));
		}
	}

	UTextureRenderTarget2D* CreateRenderTarget(FMaterialPropertyEx InProperty, const FIntPoint& InTargetSize, bool bInUsePooledRenderTargets, const FColor& BackgroundColor)
	{
		return MaterialBakingModule.CreateRenderTarget(InProperty, InTargetSize, bInUsePooledRenderTargets, BackgroundColor);
	}

	void SaveIntermediateTextures(const FBakeOutputEx& BakeOutput, const FMaterialPropertyEx& Property, const FString& FilenameString)
	{
#if WITH_EDITOR
		// If saving intermediates is turned on
		if (bSaveIntermediateTextures)
		{
			if (!BakeOutput.PropertyData[Property].IsEmpty())
			{
				static int32 SaveCount = 0;
				TRACE_CPUPROFILER_EVENT_SCOPE(SaveIntermediateTextures)
				FString TrimmedPropertyName = Property.ToString();

				const FString DirectoryPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectIntermediateDir() + TEXT("MaterialBaking/"));
				FString FullPath = FString::Printf(TEXT("%s%s-%d-%s.bmp"), *DirectoryPath, *FilenameString, SaveCount++, *TrimmedPropertyName);
				FFileHelper::CreateBitmap(*FullPath, BakeOutput.PropertySizes[Property].X, BakeOutput.PropertySizes[Property].Y, BakeOutput.PropertyData[Property].GetData());
			}
		}
#endif
	}

private:
	virtual FBakeOutputEx& GetBakeOutput(int32 InMaterialIndex) = 0;
	virtual UTextureRenderTarget2D* GetRenderTarget(FMaterialPropertyEx InMaterialProperty, const FIntPoint& InRequiredSize, const FMaterialDataEx& InMaterialSettings) = 0;
	virtual void OnMaterialPropertyBaked(const FMaterialDataEx& CurrentMaterialSettings, const FMaterialPropertyEx& Property, FMeshMaterialRenderItem* RenderItem, UTextureRenderTarget2D* RenderTarget, FExportMaterialProxy* ExportMaterialProxy, FBakeOutputEx& CurrentOutput) {}

	void ComputeMeshProcessingOrder()
	{
		ProcessingOrder.Reserve(MeshSettings.Num());
		for (int32 Index = 0; Index < MeshSettings.Num(); ++Index)
		{
			if (!MaterialSettings[Index]->PropertySizes.IsEmpty())
			{
				ProcessingOrder.Add(Index);
			}
		}

		// Start with the biggest mesh first so we can always reuse the same vertex/index buffers.
		// This will decrease the number of allocations backed by newly allocated memory from the OS,
		// which will reduce soft page faults while copying into that memory.
		// Soft page faults are now incredibly expensive on Windows 10.
		Algo::SortBy(
			ProcessingOrder,
			[this](const uint32 Index) { return MeshSettings[Index]->MeshDescription ? MeshSettings[Index]->MeshDescription->Vertices().Num() : 0; },
			TGreater<>()
		);
	}

	// This will create and prepare FMeshMaterialRenderItem for each property sizes we're going to need
	TMap<FRenderItemKey, FMeshMaterialRenderItem*>* PrepareRenderItems_AnyThread(int32 MaterialIndex)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PrepareRenderItems);

		TMap<FRenderItemKey, FMeshMaterialRenderItem*>* RenderItems = new TMap<FRenderItemKey, FMeshMaterialRenderItem*>();
		const FMaterialDataEx* CurrentMaterialSettings = MaterialSettings[MaterialIndex];
		const FMeshData* CurrentMeshSettings = MeshSettings[MaterialIndex];

		check(!CurrentMaterialSettings->PropertySizes.IsEmpty());

		for (const auto& [Property, Size] : CurrentMaterialSettings->PropertySizes)
		{
			FRenderItemKey RenderItemKey(CurrentMeshSettings, Size);
			if (RenderItems->Find(RenderItemKey) == nullptr)
			{
				RenderItems->Add(RenderItemKey, new FMeshMaterialRenderItem(Size, CurrentMeshSettings, &MaterialBakingDynamicMeshBufferAllocator));
			}
		}

		return RenderItems;
	}

	void LaunchAsyncPrepareRenderItems(int32 InLaunchCount)
	{
		int32 LastRenderItem = NextRenderItem + InLaunchCount;
		for (; NextRenderItem < NumMaterials && NextRenderItem < LastRenderItem; NextRenderItem++)
		{
			PreparedRenderItems[NextRenderItem % PipelineDepth] =
				Async(
					EAsyncExecution::ThreadPool,
					[this, Index=NextRenderItem]()
					{
						return PrepareRenderItems_AnyThread(ProcessingOrder[Index]);
					}
			);
		}
	}

	TMap<FRenderItemKey, FMeshMaterialRenderItem*>* GetRenderItems(int32 InIndex)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(WaitOnPreparedRenderItems)
		TMap<FRenderItemKey, FMeshMaterialRenderItem*>* RenderItems = PreparedRenderItems[InIndex % PipelineDepth].Get();

		// Prepare the next render item in advance
		if (NextRenderItem < NumMaterials)
		{
			check((NextRenderItem % PipelineDepth) == (InIndex % PipelineDepth));
			LaunchAsyncPrepareRenderItems(1);
		}

		return RenderItems;
	}

	void CreateMaterialProxies()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CreateMaterialProxies)

		for (int32 Index = 0; Index < NumMaterials; ++Index)
		{
			int32 MaterialIndex = ProcessingOrder[Index];
			const FMaterialDataEx* CurrentMaterialSettings = MaterialSettings[MaterialIndex];

			TArray<UTexture*> MaterialTextures;
			CurrentMaterialSettings->Material->GetUsedTextures(MaterialTextures);

			// Force load materials used by the current material
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(LoadTexturesForMaterial)

				FTextureCompilingManager::Get().FinishCompilation(MaterialTextures);

				for (UTexture* Texture : MaterialTextures)
				{
					if (UTexture2D* Texture2D = Cast<UTexture2D>(Texture))
					{
						if (Texture2D->IsStreamable())
						{
							// Force LODs in with high priority, including cinematic ones.
							Texture2D->SetForceMipLevelsToBeResident(30.f, 0xFFFFFFFF);
							Texture2D->StreamIn(FStreamableRenderResourceState::MAX_LOD_COUNT, true);
						}
					}
				}
			}

			for (TMap<FMaterialPropertyEx, FIntPoint>::TConstIterator PropertySizeIterator = CurrentMaterialSettings->PropertySizes.CreateConstIterator(); PropertySizeIterator; ++PropertySizeIterator)
			{
				// They will be stored in the pool and compiled asynchronously
				MaterialBakingModule.CreateMaterialProxy(CurrentMaterialSettings, PropertySizeIterator.Key());
			}
		}


		// Force all mip maps to load before baking the materials
		{
			const float STREAMING_WAIT_DT = 0.1f;
			while (IStreamingManager::Get().StreamAllResources(STREAMING_WAIT_DT) > 0)
			{
				// Application tick.
				FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
				FTSTicker::GetCoreTicker().Tick(FApp::GetDeltaTime());
			}
		}
	}

	void BakeMaterialProperty(const FMaterialDataEx& CurrentMaterialSettings, const FMaterialPropertyEx& Property, FMeshMaterialRenderItem* RenderItem, UTextureRenderTarget2D* RenderTarget, FExportMaterialProxy* ExportMaterialProxy, FBakeOutputEx& CurrentOutput)
	{
		// Perform everything left of the operation directly on the render thread since we need to modify some RenderItem's properties
		// for each render pass and we can't do that without costly synchronization (flush) between the game thread and render thread.
		// Everything slow to execute has already been prepared on the game thread anyway.
		ENQUEUE_RENDER_COMMAND(RenderOneMaterial)(
			[RenderItem, RenderTarget, ExportMaterialProxy](FRHICommandListImmediate& RHICmdList)
			{
				RenderCaptureInterface::FScopedCapture RenderCapture(CVarMaterialBakingRDOCCapture.GetValueOnAnyThread() == 1, &RHICmdList, TEXT("MaterialBaking"));

				FSceneViewFamily ViewFamily(FSceneViewFamily::ConstructionValues(RenderTarget->GetRenderTargetResource(), nullptr,
					FEngineShowFlags(ESFIM_Game))
					.SetTime(FGameTime()));

				RenderItem->MaterialRenderProxy = ExportMaterialProxy;
				RenderItem->ViewFamily = &ViewFamily;

				FTextureRenderTargetResource* RenderTargetResource = RenderTarget->GetRenderTargetResource();
				FCanvas Canvas(RenderTargetResource, nullptr, FGameTime::GetTimeSinceAppStart(), GMaxRHIFeatureLevel);
				Canvas.SetAllowedModes(FCanvas::Allow_Flush);
				Canvas.SetRenderTargetRect(FIntRect(0, 0, RenderTarget->GetSurfaceWidth(), RenderTarget->GetSurfaceHeight()));
				Canvas.SetBaseTransform(Canvas.CalcBaseTransform2D(RenderTarget->GetSurfaceWidth(), RenderTarget->GetSurfaceHeight()));

				// Virtual textures may require repeated rendering to warm up.
				int32 WarmupIterationCount = 1;
				if (UseVirtualTexturing(ViewFamily.GetShaderPlatform()))
				{
					const FMaterial& MeshMaterial = ExportMaterialProxy->GetIncompleteMaterialWithFallback(ViewFamily.GetFeatureLevel());
					if (!MeshMaterial.GetUniformVirtualTextureExpressions().IsEmpty())
					{
						WarmupIterationCount = CVarMaterialBakingVTWarmupFrames.GetValueOnAnyThread();
					}
				}

				// Do rendering
				for (int WarmupIndex = 0; WarmupIndex < WarmupIterationCount; ++WarmupIndex)
				{
					FCanvas::FCanvasSortElement& SortElement = Canvas.GetSortElement(Canvas.TopDepthSortKey());
					SortElement.RenderBatchArray.Add(RenderItem);
					Canvas.Flush_RenderThread(RHICmdList);
					SortElement.RenderBatchArray.Empty();

					RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
				}
			}
		);

		OnMaterialPropertyBaked(CurrentMaterialSettings, Property, RenderItem, RenderTarget, ExportMaterialProxy, CurrentOutput);
	}

protected:
	FMaterialBakingModule& MaterialBakingModule;

	const TArray<FMaterialDataEx*>& MaterialSettings;
	const TArray<FMeshData*>& MeshSettings;
	const bool bSaveIntermediateTextures;
	const bool bEmissiveHDR;

	// Distance between the command sent to rendering and the GPU read-back of the result
	// to minimize sync time waiting on GPU.
	static const int32 PipelineDepth = 16;

private:
	int32 NumMaterials;
	TArray<uint32> ProcessingOrder;

	FMaterialBakingDynamicMeshBufferAllocator MaterialBakingDynamicMeshBufferAllocator;

	// We reuse the pipeline depth to prepare render items in advance to avoid stalling the game thread
	int NextRenderItem = 0;
	TFuture<TMap<FRenderItemKey, FMeshMaterialRenderItem*>*> PreparedRenderItems[PipelineDepth];
};


class FMaterialBakingProcessorSingleOutput : public FMaterialBakingProcessor
{
public:
	FMaterialBakingProcessorSingleOutput(FMaterialBakingModule& InMaterialBakingModule, const TArray<FMaterialDataEx*>& InMaterialSettings, const TArray<FMeshData*>& InMeshSettings, FBakeOutputEx& InOutput)
		: FMaterialBakingProcessor(InMaterialBakingModule, InMaterialSettings, InMeshSettings)
		, Output(InOutput)
	{
		if (!MaterialSettings.IsEmpty())
		{
			FMaterialDataEx* DefaultMaterialData = MaterialSettings[0];

			// Single output path can only work if all materials settings share the same properties
			for (const FMaterialDataEx* MaterialData : MaterialSettings)
			{
				for (const auto& [Property, Size] : MaterialData->PropertySizes)
				{
					check(DefaultMaterialData->PropertySizes.Contains(Property) && DefaultMaterialData->PropertySizes[Property] == Size);
				}
				check(MaterialData->bPerformBorderSmear == DefaultMaterialData->bPerformBorderSmear);
				check(MaterialData->bPerformShrinking == DefaultMaterialData->bPerformShrinking);
				check(MaterialData->bTangentSpaceNormal == DefaultMaterialData->bTangentSpaceNormal);
				check(MaterialData->BlendMode == DefaultMaterialData->BlendMode);
				check(MaterialData->BackgroundColor == DefaultMaterialData->BackgroundColor);
			}

			PrepareBakeOutput(DefaultMaterialData, Output);

			// Create render targets for all properties
			for (const auto& [Property, Size] : Output.PropertySizes)
			{
				// Skip pooling, all materials will be baked to the same set of render targets
				const bool bUsePooledRenderTargets = false;

				UTextureRenderTarget2D* RenderTarget = CreateRenderTarget(Property, Size, bUsePooledRenderTargets, DefaultMaterialData->BackgroundColor);
				RenderTargets.Emplace(Property, RenderTarget);
			}
		}
	}

	~FMaterialBakingProcessorSingleOutput()
	{
		if (!MaterialSettings.IsEmpty())
		{
			// Wait until every tasks have been queued so that NumTasks is only decreasing
			FlushRenderingCommands();

			FMaterialDataEx* DefaultMaterialData = MaterialSettings[0];

			// Read back from the render targets
			for (auto& [Property, RenderTarget] : RenderTargets)
			{
				FRenderTarget* RTResource = RenderTarget->GameThread_GetRenderTargetResource();
				check(RTResource);

				if (Property.Type != MP_EmissiveColor)
				{
					RTResource->ReadPixels(Output.PropertyData[Property]);
				}
				else
				{
					TArray<FFloat16Color> OutputHDRColor;
					RTResource->ReadFloat16Pixels(OutputHDRColor);
					
					int32 Color16PitchPixels = Output.PropertySizes[Property].X;
					ProcessEmissiveOutput(OutputHDRColor.GetData(), Color16PitchPixels, Output.PropertySizes[Property], Output.PropertyData[Property], Output.EmissiveScale, DefaultMaterialData->BackgroundColor);
					
					if (bEmissiveHDR)
					{
						Output.HDRPropertyData[Property] = MoveTemp(OutputHDRColor);
					}
				}

				if (DefaultMaterialData->bPerformBorderSmear)
				{
					// This will resize the output to a single pixel if the result is monochrome.
					FMaterialBakingHelpers::PerformUVBorderSmearAndShrink(Output.PropertyData[Property], Output.PropertySizes[Property].X, Output.PropertySizes[Property].Y, DefaultMaterialData->BackgroundColor);
				}

				SaveIntermediateTextures(Output, Property, TEXT("Final"));
			}
		}
	}

	virtual FBakeOutputEx& GetBakeOutput(int32 InMaterialIndex) override
	{
		return Output;
	}

	virtual UTextureRenderTarget2D* GetRenderTarget(FMaterialPropertyEx InMaterialProperty, const FIntPoint& InRequiredSize, const FMaterialDataEx& InMaterialSettings) override
	{
		return RenderTargets[InMaterialProperty].Get();
	}

private:
	TMap<FMaterialPropertyEx, TStrongObjectPtr<UTextureRenderTarget2D>> RenderTargets;
	FBakeOutputEx& Output;
};


class FMaterialBakingProcessorMultiOutput : public FMaterialBakingProcessor
{
public:
	FMaterialBakingProcessorMultiOutput(FMaterialBakingModule& InMaterialBakingModule, const TArray<FMaterialDataEx*>& InMaterialSettings, const TArray<FMeshData*>& InMeshSettings, TArray<FBakeOutputEx>& InOutput)
		: FMaterialBakingProcessor(InMaterialBakingModule, InMaterialSettings, InMeshSettings)
		, Output(InOutput)
	{
		Output.SetNum(InMaterialSettings.Num());

		int32 NumOutputs = InOutput.Num();
		if (NumOutputs != 0)
		{
			if (ensure(NumOutputs == InMaterialSettings.Num()))
			{
				for (int32 Idx = 0; Idx < NumOutputs; ++Idx)
				{
					PrepareBakeOutput(InMaterialSettings[Idx], Output[Idx]);
				}
			}
		}
	}
	
	~FMaterialBakingProcessorMultiOutput()
	{
		ENQUEUE_RENDER_COMMAND(ProcessRemainingReads)(
		[this, InPipelineIndex=PipelineIndex](FRHICommandListImmediate& RHICmdList)
		{
			// Enqueue remaining reads
			for (int32 Index = 0; Index < PipelineDepth; Index++)
			{
				int32 LocalPipelineIndex = (InPipelineIndex + Index) % PipelineDepth;

				if (PipelineContext[LocalPipelineIndex].ReadCommand)
				{
					PipelineContext[LocalPipelineIndex].ReadCommand(RHICmdList);
				}
			}
		});

		// Wait until every tasks have been queued so that NumTasks is only decreasing
		FlushRenderingCommands();

		// Wait for any remaining final processing tasks
		while (NumTasks.Load(EMemoryOrder::Relaxed) > 0)
		{
			FPlatformProcess::Sleep(0.1f);
		}

		// Wait for all tasks to have been processed before clearing the staging buffers
		FlushRenderingCommands();

		ENQUEUE_RENDER_COMMAND(ClearStagingBufferPool)(
			[this](FRHICommandListImmediate& RHICmdList)
			{
				StagingBufferPool.Clear_RenderThread(RHICmdList);
			}
		);

		// Wait for StagingBufferPool clear to have executed before exiting the function
		FlushRenderingCommands();
	}

private:
	virtual void OnMaterialPropertyBaked(const FMaterialDataEx& CurrentMaterialSettings, const FMaterialPropertyEx& Property, FMeshMaterialRenderItem* RenderItem, UTextureRenderTarget2D* RenderTarget, FExportMaterialProxy* ExportMaterialProxy, FBakeOutputEx& CurrentOutput) override
	{
		ENQUEUE_RENDER_COMMAND(CopyStagingBuffer)(
			[this, RenderTarget, Property, ExportMaterialProxy, &CurrentMaterialSettings, &CurrentOutput, InPipelineIndex=PipelineIndex](FRHICommandListImmediate& RHICmdList)
			{
				FTextureRenderTargetResource* RenderTargetResource = RenderTarget->GetRenderTargetResource();

				FTextureRHIRef StagingBufferRef = StagingBufferPool.CreateStagingBuffer_RenderThread(RHICmdList, RenderTargetResource->GetSizeX(), RenderTargetResource->GetSizeY(), RenderTarget->GetFormat(), RenderTarget->IsSRGB());
				FGPUFenceRHIRef GPUFence = RHICreateGPUFence(TEXT("MaterialBackingFence"));
				TransitionAndCopyTexture(RHICmdList, RenderTargetResource->GetRenderTargetTexture(), StagingBufferRef, {});
				RHICmdList.WriteGPUFence(GPUFence);

				// Prepare a lambda for final processing that will be executed asynchronously
				NumTasks++;
				auto FinalProcessing_AnyThread =
					[this, CurrentMaterialSettings, &CurrentOutput, Property](FTextureRHIRef& StagingBuffer, void* Data, int32 DataWidth, int32 DataHeight)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(FinalProcessing)

						TArray<FColor>& OutputColor = CurrentOutput.PropertyData[Property];
						FIntPoint& OutputSize = CurrentOutput.PropertySizes[Property];

						OutputColor.SetNum(OutputSize.X * OutputSize.Y);

						if (Property.Type == MP_EmissiveColor)
						{
							// Only one thread will write to CurrentOutput.EmissiveScale since there can be only one emissive channel property per FBakeOutputEx
							ProcessEmissiveOutput((const FFloat16Color*)Data, DataWidth, OutputSize, OutputColor, CurrentOutput.EmissiveScale, CurrentMaterialSettings.BackgroundColor);

							if (bEmissiveHDR)
							{
								TArray<FFloat16Color>& OutputHDRColor = CurrentOutput.HDRPropertyData[Property];
								OutputHDRColor.SetNum(OutputSize.X * OutputSize.Y);
								ConvertRawR16G16B16A16FDataToFFloat16Color(OutputSize.X, OutputSize.Y, (uint8*)Data, DataWidth * sizeof(FFloat16Color), OutputHDRColor.GetData());
							}
						}
						else
						{
							TRACE_CPUPROFILER_EVENT_SCOPE(ConvertRawB8G8R8A8DataToFColor)

							check(StagingBuffer->GetFormat() == PF_B8G8R8A8);
							ConvertRawB8G8R8A8DataToFColor(OutputSize.X, OutputSize.Y, (uint8*)Data, DataWidth * sizeof(FColor), OutputColor.GetData());
						}

						// We can't unmap ourself since we're not on the render thread
						StagingBufferPool.ReleaseStagingBufferForUnmap_AnyThread(StagingBuffer);

						if (CurrentMaterialSettings.bPerformShrinking)
						{
							FMaterialBakingHelpers::PerformShrinking(OutputColor, OutputSize.X, OutputSize.Y, CurrentMaterialSettings.BackgroundColor);
						}

						if (CurrentMaterialSettings.bPerformBorderSmear)
						{
							// This will resize the output to a single pixel if the result is monochrome.
							FMaterialBakingHelpers::PerformUVBorderSmearAndShrink(OutputColor, OutputSize.X, OutputSize.Y, CurrentMaterialSettings.BackgroundColor);
						}
	
						SaveIntermediateTextures(CurrentOutput, Property, *CurrentMaterialSettings.Material->GetName());
	
						NumTasks--;
					};

				// Run previous command if we're going to overwrite it meaning pipeline depth has been reached
				if (PipelineContext[InPipelineIndex].ReadCommand)
				{
					PipelineContext[InPipelineIndex].ReadCommand(RHICmdList);
				}

				// Generate a texture reading command that will be executed once it reaches the end of the pipeline
				PipelineContext[InPipelineIndex].ReadCommand =
					[FinalProcessing_AnyThread, StagingBufferRef = MoveTemp(StagingBufferRef), GPUFence = MoveTemp(GPUFence)](FRHICommandListImmediate& RHICmdList) mutable
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(MapAndEnqueue)

						void* Data = nullptr;
						int32 Width; int32 Height;
						RHICmdList.MapStagingSurface(StagingBufferRef, GPUFence.GetReference(), Data, Width, Height);

						// Schedule the copy and processing on another thread to free up the render thread as much as possible
						Async(
							EAsyncExecution::ThreadPool,
							[FinalProcessing_AnyThread, Data, Width, Height, StagingBufferRef = MoveTemp(StagingBufferRef)]() mutable
							{
								FinalProcessing_AnyThread(StagingBufferRef, Data, Width, Height);
							}
						);
					};
			}
		);

		PipelineIndex = (PipelineIndex + 1) % PipelineDepth;
	}

	virtual FBakeOutputEx& GetBakeOutput(int32 InMaterialIndex) override
	{
		return Output[InMaterialIndex];
	}

	virtual UTextureRenderTarget2D* GetRenderTarget(FMaterialPropertyEx InMaterialProperty, const FIntPoint& InRequiredSize, const FMaterialDataEx& InMaterialSettings) override
	{
		// It is safe to reuse the same render targets for each draw pass since they all execute sequentially on the GPU and are copied to staging buffers before
		// being reused.
		const bool bUsePooledRenderTargets = true;
		return CreateRenderTarget(InMaterialProperty, InRequiredSize, bUsePooledRenderTargets, InMaterialSettings.BackgroundColor);
	}

private:
	struct FPipelineContext
	{
		typedef TFunction<void(FRHICommandListImmediate& RHICmdList)> FReadCommand;
		FReadCommand ReadCommand;
	};

	// Distance between the command sent to rendering and the GPU read-back of the result
	// to minimize sync time waiting on GPU.
	int32 PipelineIndex = 0;
	FPipelineContext PipelineContext[PipelineDepth];

	TAtomic<uint32>    NumTasks = 0;
	FStagingBufferPool StagingBufferPool;

	TArray<FBakeOutputEx>& Output;
};

void FMaterialBakingModule::BakeMaterials(const TArray<FMaterialDataEx*>& MaterialSettings, const TArray<FMeshData*>& MeshSettings, FBakeOutputEx& Output)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialBakingModule::BakeMaterials)

	TGuardValue<bool> GuardIsBaking(bIsBakingMaterials, true);

	FMaterialBakingProcessorSingleOutput MaterialBakingProcessor(*this, MaterialSettings, MeshSettings, Output);
	MaterialBakingProcessor.BakeMaterials();
}


void FMaterialBakingModule::BakeMaterials(const TArray<FMaterialDataEx*>& MaterialSettings, const TArray<FMeshData*>& MeshSettings, TArray<FBakeOutputEx>& Output)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialBakingModule::BakeMaterials)

	TGuardValue<bool> GuardIsBaking(bIsBakingMaterials, true);

	FMaterialBakingProcessorMultiOutput MaterialBakingProcessor(*this, MaterialSettings, MeshSettings, Output);
	MaterialBakingProcessor.BakeMaterials();
}

bool FMaterialBakingModule::SetupMaterialBakeSettings(TArray<TWeakObjectPtr<UObject>>& OptionObjects, int32 NumLODs)
{
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("WindowTitle", "Material Baking Options"))
		.SizingRule(ESizingRule::Autosized);

	TSharedPtr<SMaterialOptions> Options;

	Window->SetContent
	(
		SAssignNew(Options, SMaterialOptions)
		.WidgetWindow(Window)
		.NumLODs(NumLODs)
		.SettingsObjects(OptionObjects)
	);

	TSharedPtr<SWindow> ParentWindow;
	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
		FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);
		return !Options->WasUserCancelled();
	}

	return false;
}

void FMaterialBakingModule::SetEmissiveHDR(bool bHDR)
{
	bEmissiveHDR = bHDR;
}

void FMaterialBakingModule::SetLinearBake(bool bCorrectLinear)
{
	// PerPropertyGamma ultimately sets whether the render target is linear
	PerPropertyColorSpace.Reset();
	if (bCorrectLinear)
	{
		DefaultColorSpace = EPropertyColorSpace::Linear;
		PerPropertyColorSpace.Add(MP_BaseColor, EPropertyColorSpace::sRGB);
		PerPropertyColorSpace.Add(MP_SubsurfaceColor, EPropertyColorSpace::sRGB);
		PerPropertyColorSpace.Add(FMaterialPropertyEx::TransmittanceColor, EPropertyColorSpace::sRGB);
	}
	else
	{
		DefaultColorSpace = EPropertyColorSpace::sRGB;
		PerPropertyColorSpace.Add(MP_EmissiveColor, EPropertyColorSpace::Linear); // Always linear because it uses HDR
		PerPropertyColorSpace.Add(MP_Normal, EPropertyColorSpace::Linear);
		PerPropertyColorSpace.Add(MP_Refraction, EPropertyColorSpace::Linear);
		PerPropertyColorSpace.Add(MP_Opacity, EPropertyColorSpace::Linear);
		PerPropertyColorSpace.Add(MP_OpacityMask, EPropertyColorSpace::Linear);
		PerPropertyColorSpace.Add(MP_ShadingModel, EPropertyColorSpace::Linear);
		PerPropertyColorSpace.Add(FMaterialPropertyEx::ClearCoatBottomNormal, EPropertyColorSpace::Linear);
	}
}

bool FMaterialBakingModule::IsLinearBake(FMaterialPropertyEx Property)
{
	const EPropertyColorSpace* OverrideColorSpace = PerPropertyColorSpace.Find(Property);
	const EPropertyColorSpace ColorSpace = OverrideColorSpace ? *OverrideColorSpace : DefaultColorSpace;
	return ColorSpace == EPropertyColorSpace::Linear;
}

void FMaterialBakingModule::CleanupMaterialProxies()
{
	TArray<FMaterial*> ResourcesToFree;
	for (auto Iterator : MaterialProxyPool)
	{
		ResourcesToFree.Add(Iterator.Value.Value);
	}
	FMaterial::DeferredDeleteArray(ResourcesToFree);

	MaterialProxyPool.Reset();
}

void FMaterialBakingModule::CleanupRenderTargets()
{
	RenderTargetPool.Empty();
}

UTextureRenderTarget2D* FMaterialBakingModule::CreateRenderTarget(FMaterialPropertyEx InProperty, const FIntPoint& InTargetSize, bool bInUsePooledRenderTargets, const FColor& BackgroundColor)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialBakingModule::CreateRenderTarget)

	// Lookup gamma and format settings for property, if not found use default values

	// Gamma
	const EPropertyColorSpace* OverrideColorSpace = PerPropertyColorSpace.Find(InProperty);
	const EPropertyColorSpace ColorSpace = OverrideColorSpace ? *OverrideColorSpace : DefaultColorSpace;
	const bool bForceLinearGamma = ColorSpace == EPropertyColorSpace::Linear;

	// Pixel format
	const EPixelFormat PixelFormat = PerPropertyFormat.Contains(InProperty) ? PerPropertyFormat[InProperty] : PF_B8G8R8A8;

	const int32 MaxTextureSize = 1 << (MAX_TEXTURE_MIP_COUNT - 1); // Don't use GetMax2DTextureDimension() as this is for the RHI only.
	const FIntPoint ClampedTargetSize(FMath::Clamp(InTargetSize.X, 1, MaxTextureSize), FMath::Clamp(InTargetSize.Y, 1, MaxTextureSize));

	UTextureRenderTarget2D* RenderTarget = nullptr;

	// First, look in pool
	if (bInUsePooledRenderTargets)
	{
		auto RenderTargetComparison = [bForceLinearGamma, PixelFormat, ClampedTargetSize](const TStrongObjectPtr<UTextureRenderTarget2D>& CompareRenderTarget) -> bool
		{
			return (CompareRenderTarget->SizeX == ClampedTargetSize.X && CompareRenderTarget->SizeY == ClampedTargetSize.Y && CompareRenderTarget->OverrideFormat == PixelFormat && CompareRenderTarget->bForceLinearGamma == bForceLinearGamma);
		};

		// Find any pooled render target with suitable properties.
		TStrongObjectPtr<UTextureRenderTarget2D>* FindResult = RenderTargetPool.FindByPredicate(RenderTargetComparison);
		if (FindResult)
		{
			RenderTarget = FindResult->Get();
		}
	}

	// If we want to avoid pooling, or no render target was found in the pool, create a new one
	if (!RenderTarget)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CreateNewRenderTarget)

		// Not found - create a new one.
		RenderTarget = NewObject<UTextureRenderTarget2D>();
		check(RenderTarget);

		RenderTarget->ClearColor = bForceLinearGamma ? BackgroundColor.ReinterpretAsLinear() : FLinearColor(BackgroundColor);
		RenderTarget->TargetGamma = 0.0f;
		RenderTarget->InitCustomFormat(ClampedTargetSize.X, ClampedTargetSize.Y, PixelFormat, bForceLinearGamma);

		if (bInUsePooledRenderTargets)
		{
			RenderTargetPool.Emplace(RenderTarget);
		}
	}

	const bool bClearRenderTarget = true;
	RenderTarget->UpdateResourceImmediate(bClearRenderTarget);

	checkf(RenderTarget != nullptr, TEXT("Unable to create or find valid render target"));
	return RenderTarget;
}

FExportMaterialProxy* FMaterialBakingModule::CreateMaterialProxy(const FMaterialDataEx* MaterialSettings, const FMaterialPropertyEx& Property)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialBakingModule::CreateMaterialProxy)

	FExportMaterialProxy* Proxy = nullptr;

	// Find all pooled material proxy matching this material
	TArray<FMaterialPoolValue> Entries;
	MaterialProxyPool.MultiFind(MaterialSettings->Material, Entries);

	// Look for the matching property
	for (FMaterialPoolValue& Entry : Entries)
	{
		if (Entry.Key == Property && Entry.Value->bTangentSpaceNormal == MaterialSettings->bTangentSpaceNormal && Entry.Value->ProxyBlendMode == MaterialSettings->BlendMode)
		{
			Proxy = Entry.Value;
			break;
		}
	}

	// Not found, create a new entry
	if (Proxy == nullptr)
	{
		Proxy = new FExportMaterialProxy(MaterialSettings->Material, Property.Type, Property.CustomOutput.ToString(), false /* bInSynchronousCompilation */, MaterialSettings->bTangentSpaceNormal, MaterialSettings->BlendMode, false);
		MaterialProxyPool.Add(MaterialSettings->Material, FMaterialPoolValue(Property, Proxy));
	}

	return Proxy;
}

void ProcessEmissiveOutput(const FFloat16Color* Color16, int32 Color16PitchPixels, const FIntPoint& OutputSize, TArray<FColor>& OutputColor, float& EmissiveScale, const FColor& BackgroundColor)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialBakingModule::ProcessEmissiveOutput)

	const int32 NumThreads = [&]()
	{
		return FPlatformProcess::SupportsMultithreading() ? FPlatformMisc::NumberOfCores() : 1;
	}();
	const int32 LinesPerThread = FMath::CeilToInt((float)OutputSize.Y / (float)NumThreads);
	const FFloat16Color BackgroundColor16 = FFloat16Color(FLinearColor(BackgroundColor)); // Can assume emissive always uses sRGB
	const bool bShouldNormalize = CVarMaterialBakingForceDisableEmissiveScaling.GetValueOnAnyThread() == 0;
	float GlobalMaxValue = 1.0f;

	if (bShouldNormalize)
	{
		float* MaxValue = new float[NumThreads];
		FMemory::Memset(MaxValue, 0, NumThreads * sizeof(MaxValue[0]));
		
		// Find maximum float value across texture
		ParallelFor(NumThreads, [&Color16, LinesPerThread, MaxValue, OutputSize, Color16PitchPixels, BackgroundColor16](int32 Index)
		{
			const int32 EndY = FMath::Min((Index + 1) * LinesPerThread, OutputSize.Y);			
			float& CurrentMaxValue = MaxValue[Index];
			for (int32 PixelY = Index * LinesPerThread; PixelY < EndY; ++PixelY)
			{
				const int32 SrcYOffset = PixelY * Color16PitchPixels;
				for (int32 PixelX = 0; PixelX < OutputSize.X; PixelX++)
				{
					const FFloat16Color& Pixel16 = Color16[PixelX + SrcYOffset];
					// Find maximum channel value across texture
					if (!(Pixel16 == BackgroundColor16))
					{
						CurrentMaxValue = FMath::Max(CurrentMaxValue, FMath::Max3(Pixel16.R.GetFloat(), Pixel16.G.GetFloat(), Pixel16.B.GetFloat()));
					}
				}
			}
		});

		GlobalMaxValue = [&MaxValue, NumThreads]
		{
			float TempValue = 0.0f;
			for (int32 ThreadIndex = 0; ThreadIndex < NumThreads; ++ThreadIndex)
			{
				TempValue = FMath::Max(TempValue, MaxValue[ThreadIndex]);
			}

			return TempValue;
		}();
		
		if (GlobalMaxValue <= 0.01f)
		{
			// Black emissive, no need to scale
			GlobalMaxValue = 1.0f;
		}
	}

	// Now convert Float16 to Color using the scale
	OutputColor.SetNumUninitialized(OutputSize.X * OutputSize.Y);
	const float Scale = 255.0f / GlobalMaxValue;
	ParallelFor(NumThreads, [&Color16, LinesPerThread, &OutputColor, OutputSize, Color16PitchPixels, Scale, BackgroundColor16, BackgroundColor](int32 Index)
	{
		const int32 EndY = FMath::Min((Index + 1) * LinesPerThread, OutputSize.Y);
		for (int32 PixelY = Index * LinesPerThread; PixelY < EndY; ++PixelY)
		{
			const int32 SrcYOffset = PixelY * Color16PitchPixels;
			const int32 DstYOffset = PixelY * OutputSize.X;

			for (int32 PixelX = 0; PixelX < OutputSize.X; PixelX++)
			{
				const FFloat16Color& Pixel16 = Color16[PixelX + SrcYOffset];
				FColor& Pixel8 = OutputColor[PixelX + DstYOffset];

				if (Pixel16 == BackgroundColor16)
				{
					Pixel8 = BackgroundColor;
				}
				else
				{
					Pixel8.R = (uint8)FMath::Clamp(FMath::RoundToInt(Pixel16.R.GetFloat() * Scale), 0, 255);
					Pixel8.G = (uint8)FMath::Clamp(FMath::RoundToInt(Pixel16.G.GetFloat() * Scale), 0, 255);
					Pixel8.B = (uint8)FMath::Clamp(FMath::RoundToInt(Pixel16.B.GetFloat() * Scale), 0, 255);
				}
					
				Pixel8.A = 255;
			}
		}
	});

	// This scale will be used in the proxy material to get the original range of emissive values outside of 0-1
	EmissiveScale = GlobalMaxValue;
}

void FMaterialBakingModule::OnObjectModified(UObject* Object)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialBakingModule::OnObjectModified)

	if (CVarUseMaterialProxyCaching.GetValueOnAnyThread())
	{
		UMaterialInterface* MaterialToInvalidate = Cast<UMaterialInterface>(Object);
		if (!MaterialToInvalidate)
		{
			// Check to see if the object is a material editor instance constant and if so, retrieve its source instance
			UMaterialEditorInstanceConstant* EditorInstance = Cast<UMaterialEditorInstanceConstant>(Object);
			if (EditorInstance && EditorInstance->SourceInstance)
			{
				MaterialToInvalidate = EditorInstance->SourceInstance;
			}
		}

		if (MaterialToInvalidate)
		{
			// Search our proxy pool for materials or material instances that refer to MaterialToInvalidate
			for (auto It = MaterialProxyPool.CreateIterator(); It; ++It)
			{
				TWeakObjectPtr<UMaterialInterface> PoolMaterialPtr = It.Key();

				// Remove stale entries from the pool
				bool bMustDelete = PoolMaterialPtr.IsValid();
				if (!bMustDelete)
				{
					bMustDelete = PoolMaterialPtr == MaterialToInvalidate;
				}

				// No match - Test the MaterialInstance hierarchy
				if (!bMustDelete)
				{
					UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(PoolMaterialPtr);
					while (!bMustDelete && MaterialInstance && MaterialInstance->Parent != nullptr)
					{
						bMustDelete = MaterialInstance->Parent == MaterialToInvalidate;
						MaterialInstance = Cast<UMaterialInstance>(MaterialInstance->Parent);
					}
				}

				// We have a match, remove the entry from our pool
				if (bMustDelete)
				{
					TArray<FMaterial*> ResourcesToFree;
					ResourcesToFree.Add(It.Value().Value);
					FMaterial::DeferredDeleteArray(ResourcesToFree);

					It.RemoveCurrent();
				}
			}
		}
	}
}

void FMaterialBakingModule::OnPreGarbageCollect()
{
	// Do not cleanup material proxies while baking materials.
	if (!bIsBakingMaterials)
	{
		CleanupMaterialProxies();
	}
}

#undef LOCTEXT_NAMESPACE //"MaterialBakingModule"
