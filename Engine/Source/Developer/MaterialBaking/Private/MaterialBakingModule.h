// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TextureRenderTarget2D.h"
#include "UObject/StrongObjectPtr.h"
#include "Modules/ModuleInterface.h"
#include "PixelFormat.h"
#include "SceneTypes.h"
#include "IMaterialBakingModule.h"
#include "MaterialPropertyEx.h"

#define UE_API MATERIALBAKING_API

class UTextureRenderTarget2D;
class UMaterialOptions;
class UMaterialInterface;

class FExportMaterialProxy;
class FTextureRenderTargetResource;

struct FMaterialData; 
struct FMeshData;
struct FBakeOutput;

class FMaterialBakingModule : public IMaterialBakingModule
{
public:
	/** IModuleInterface overrides begin */
	UE_API virtual void StartupModule();
	UE_API virtual void ShutdownModule();
	/** IModuleInterface overrides end */

	/** Bakes out material properties according to MaterialSettings using MeshSettings and stores the output in Output */
	UE_API virtual void BakeMaterials(const TArray<FMaterialData*>& MaterialSettings, const TArray<FMeshData*>& MeshSettings, TArray<FBakeOutput>& Output) override;

	/** Bakes out material properties according to extended MaterialSettings using MeshSettings and stores the output in Output */
	UE_API virtual void BakeMaterials(const TArray<FMaterialDataEx*>& MaterialSettings, const TArray<FMeshData*>& MeshSettings, TArray<FBakeOutputEx>& Output) override;

	/** Bakes out material properties according to MaterialSettings using MeshSettings and stores the output in Output */
	UE_API virtual void BakeMaterials(const TArray<FMaterialData*>& MaterialSettings, const TArray<FMeshData*>& MeshSettings, FBakeOutput& Output) override;

	/** Bakes out material properties according to extended MaterialSettings using MeshSettings and stores the output in Output */
	UE_API virtual void BakeMaterials(const TArray<FMaterialDataEx*>& MaterialSettings, const TArray<FMeshData*>& MeshSettings, FBakeOutputEx& Output) override;

	/** Promps a slate window to allow the user to populate specific material baking settings used while baking out materials */
	UE_API virtual bool SetupMaterialBakeSettings(TArray<TWeakObjectPtr<UObject>>& OptionObjects, int32 NumLODs) override;

	/** Outputs true HDR version of emissive color */
	UE_API virtual void SetEmissiveHDR(bool bHDR) override;

	/** Bakes all material properties to linear textures, except for non-emissive colors */
	UE_API virtual void SetLinearBake(bool bCorrectLinear) override;

	/** Returns whether a specific material property is baked to a linear texture or not */
	UE_API virtual bool IsLinearBake(FMaterialPropertyEx Property) override;

	/** Obtain a CRC than can help trigger a rebake if code/global settings impacting the bake result change */
	UE_API virtual uint32 GetCRC() const override;

protected:
	friend class FMaterialBakingProcessor;

	/* Creates and adds or reuses a RenderTarget from the pool */
	UE_API UTextureRenderTarget2D* CreateRenderTarget(FMaterialPropertyEx InProperty, const FIntPoint& InTargetSize, bool bInUsePooledRenderTargets, const FColor& BackgroundColor);

	/* Creates and adds (or reuses a ExportMaterialProxy from the pool if MaterialBaking.UseMaterialProxyCaching is set to 1) */
	UE_API FExportMaterialProxy* CreateMaterialProxy(const FMaterialDataEx* MaterialSettings, const FMaterialPropertyEx& Property);

	/** Cleans up all cached material proxies in MaterialProxyPool */
	UE_API void CleanupMaterialProxies();

	/** Free up all pooled render targets */
	UE_API void CleanupRenderTargets();

	/** Callback for modified objects which should be removed from MaterialProxyPool in that case */
	UE_API void OnObjectModified(UObject* Object);

	/** Callback used to clear material proxy cache on garbage collection */
	UE_API void OnPreGarbageCollect();

private:
	enum EPropertyColorSpace
	{
		Linear,
		sRGB,
	};

	/** Pool of available render targets, cached for re-using on consecutive property rendering */
	TArray<TStrongObjectPtr<UTextureRenderTarget2D>> RenderTargetPool;

	/** Pool of cached material proxies to optimize material baking workflow, stays resident when MaterialBaking.UseMaterialProxyCaching is set to 1 */
	typedef TWeakObjectPtr<UMaterialInterface>				FMaterialPoolKey;
	typedef TPair<FMaterialPropertyEx, FExportMaterialProxy*> FMaterialPoolValue;
	typedef TMultiMap<FMaterialPoolKey, FMaterialPoolValue, FDefaultSetAllocator, TWeakObjectPtrMapKeyFuncs<FMaterialPoolKey, FMaterialPoolValue, true /*bInAllowDuplicateKeys*/>> FMaterialPoolMap;
	FMaterialPoolMap MaterialProxyPool;

	/** Pixel formats to use for baking out specific material properties */
	TMap<FMaterialPropertyEx, EPixelFormat> PerPropertyFormat;

	/** Whether or not to enforce gamma correction while baking out specific material properties */
	TMap<FMaterialPropertyEx, EPropertyColorSpace> PerPropertyColorSpace;

	EPropertyColorSpace DefaultColorSpace;

	bool bEmissiveHDR;

	bool bIsBakingMaterials;
};

#undef UE_API
