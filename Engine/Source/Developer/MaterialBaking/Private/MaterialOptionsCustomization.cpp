// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialOptionsCustomization.h"
#include "MaterialOptions.h"
#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "RHI.h"

#include "IPropertyTypeCustomization.h"
#include "IPropertyUtilities.h"

static const TArray<EMaterialProperty> DisabledProperties = { MP_Refraction, MP_FrontMaterial };

TSharedRef<IPropertyTypeCustomization> FPropertyEntryCustomization::MakeInstance()
{
	return MakeShareable(new FPropertyEntryCustomization);
}

FPropertyEntryCustomization::FPropertyEntryCustomization()
{
	PropertyRestriction = MakeShareable(new FPropertyRestriction(FText::FromString("Property already set on for a different entry")));
	CurrentOptions = nullptr;
}

void FPropertyEntryCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> MaterialPropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPropertyEntry, Property));
	HeaderRow.NameContent()
	[
		MaterialPropertyHandle->CreatePropertyValueWidget()
	];

	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = CustomizationUtils.GetPropertyUtilities()->GetSelectedObjects();

	// Try and find material options instance in currently edited objects
	const TWeakObjectPtr<UObject>* MaterialOptionsPtr = SelectedObjects.FindByPredicate([](TWeakObjectPtr<UObject> Object) { return Cast<UMaterialOptions>(Object.Get()); });
	CurrentOptions = MaterialOptionsPtr ? Cast<UMaterialOptions>((MaterialOptionsPtr)->Get()) : nullptr;
	
	const int32 Index = PropertyHandle->GetIndexInArray();

	// Add restriction to ensure the user cannot set up two entries with the same EMaterialProperty value	
	MaterialPropertyHandle->AddRestriction(PropertyRestriction.ToSharedRef());

	// Set Parent handle on property change to update restrictions across all the entry properties
	TSharedPtr<IPropertyHandle> ParentHandle = PropertyHandle->GetParentHandle();
	if (ParentHandle.IsValid())
	{
		TSharedPtr<IPropertyHandle> TopParentHandle = ParentHandle->GetParentHandle();
		if (TopParentHandle.IsValid())
		{
			TopParentHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FPropertyEntryCustomization::UpdateRestrictions, Index));
		}
	}

	UpdateRestrictions(Index);
}

void FPropertyEntryCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> CustomSizePropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPropertyEntry, CustomSize));
	IDetailPropertyRow& SizeRow = ChildBuilder.AddProperty(CustomSizePropertyHandle.ToSharedRef());

	AddTextureSizeClamping(CustomSizePropertyHandle);

	TSharedPtr<IPropertyHandle> ConstantValuePropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPropertyEntry, ConstantValue));
	IDetailPropertyRow& ConstantValueRow = ChildBuilder.AddProperty(ConstantValuePropertyHandle.ToSharedRef());
}

void FPropertyEntryCustomization::UpdateRestrictions(const int32 EntryIndex)
{	
	PropertyRestriction->RemoveAll();
	const UEnum* PropertyEnum = StaticEnum<EMaterialProperty>();
	if (CurrentOptions)
	{	
		// Add all previously set material properties to be disabled
		for (int32 Index = 0; Index < CurrentOptions->Properties.Num(); ++Index)
		{
			if (Index != EntryIndex)
			{
				const FPropertyEntry& Entry = CurrentOptions->Properties[Index];
				PropertyRestriction->AddDisabledValue(PropertyEnum->GetNameStringByValue(Entry.Property));
			}
		}
	}

	for (const EMaterialProperty& Property : DisabledProperties)
	{
		PropertyRestriction->AddDisabledValue(PropertyEnum->GetNameStringByValue(Property));
	}

	PropertyRestriction->AddHiddenValue(GET_ENUMERATOR_NAME_CHECKED(EMaterialProperty, MP_WorldPositionOffset).ToString());
}

TSharedRef<IDetailCustomization> FMaterialOptionsCustomization::MakeInstance(int32 InNumLODs)
{
	return MakeShareable(new FMaterialOptionsCustomization(InNumLODs));
}

FMaterialOptionsCustomization::FMaterialOptionsCustomization(int32 InNumLODs)
	: NumLODs(InNumLODs)
{}

void FMaterialOptionsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	const FName CategoryName = FName(TEXT("MeshSettings"));
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(CategoryName);

	// Add custom LOD index selection row
	FDetailWidgetRow& LODsRow = CategoryBuilder.AddCustomRow(FText::FromString(TEXT("LODs")));
	LODsRow.NameContent()
	[
		SNew(STextBlock)
		.Text(FText::FromString(TEXT("LODs")))
		.Font(DetailBuilder.GetDetailFont())
	];

	TSharedPtr<SHorizontalBox> ContentBox;
	LODsRow.ValueContent()
	[
		SAssignNew(ContentBox, SHorizontalBox)
	];
	
	TArray<TWeakObjectPtr<UObject>> WeakObjects;
	DetailBuilder.GetObjectsBeingCustomized(WeakObjects);
	// Try and find material options instance in currently edited objects
	UMaterialOptions* CurrentOptions = Cast<UMaterialOptions>((WeakObjects.FindByPredicate([](TWeakObjectPtr<UObject> Object) { return Cast<UMaterialOptions>(Object.Get()); }))->Get());

	TSharedRef<IPropertyHandle> TextureSizePropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMaterialOptions, TextureSize));
	if (TextureSizePropertyHandle->IsValidHandle())
	{
		AddTextureSizeClamping(TextureSizePropertyHandle);
	}

	TSharedRef<IPropertyHandle> PropertiesHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMaterialOptions, Properties));
	if (PropertiesHandle->IsValidHandle())
	{
		// Setup delegate for when the number of Material Property items changes
		FSimpleDelegate RefreshDelegate = FSimpleDelegate::CreateLambda([this, &DetailBuilder, CurrentOptions]()
		{
			TArray<EMaterialProperty> Properties;
			// Ensure that we set duplicate entries to MP_MAX
			for (FPropertyEntry& Entry : CurrentOptions->Properties)
			{
				if (Properties.Contains(Entry.Property))
				{
					Entry.Property = MP_MAX;
				}
				else if (Entry.Property != MP_MAX && !DisabledProperties.Contains(Entry.Property))
				{
					Properties.Add(Entry.Property);
				}
			}
			
			DetailBuilder.ForceRefreshDetails();
		});
		PropertiesHandle->SetOnPropertyValueChanged(RefreshDelegate);
	}

	// Only allow changes to LOD indices if we have a valid options instance and if there is actually more than one index
	ContentBox->SetEnabled(TAttribute<bool>::Create([this, CurrentOptions]() -> bool { return NumLODs > 1 && CurrentOptions != nullptr; }));
	for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
	{
		ContentBox->AddSlot()
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
		.AutoWidth()
		[
			SNew(SCheckBox)
			.IsChecked(LODIndex == 0 ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
			.OnCheckStateChanged_Lambda([=](ECheckBoxState NewState)
			{
				if (NewState == ECheckBoxState::Unchecked)
				{
					CurrentOptions->LODIndices.Remove(LODIndex);
				}
				else
				{
					CurrentOptions->LODIndices.Add(LODIndex);
				}
			})
		];

		ContentBox->AddSlot()
		.Padding(FMargin(3.0f, 2.0f, 4.0f, 0.0f))
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(FText::FromString(FString::FromInt(LODIndex)))
			.Font(DetailBuilder.GetDetailFont())
		];
	}
}

void AddTextureSizeClamping(TSharedPtr<IPropertyHandle> TextureSizeProperty)
{
	TSharedPtr<IPropertyHandle> PropertyX = TextureSizeProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIntPoint, X));
	TSharedPtr<IPropertyHandle> PropertyY = TextureSizeProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIntPoint, Y));

	const FString MaxTextureResolutionString = FString::FromInt(GetMax2DTextureDimension());
	TextureSizeProperty->GetProperty()->SetMetaData(TEXT("ClampMax"), *MaxTextureResolutionString);
	TextureSizeProperty->GetProperty()->SetMetaData(TEXT("UIMax"), *MaxTextureResolutionString);
	PropertyX->SetInstanceMetaData(TEXT("ClampMax"), *MaxTextureResolutionString);
	PropertyX->SetInstanceMetaData(TEXT("UIMax"), *MaxTextureResolutionString);
	PropertyY->SetInstanceMetaData(TEXT("ClampMax"), *MaxTextureResolutionString);
	PropertyY->SetInstanceMetaData(TEXT("UIMax"), *MaxTextureResolutionString);

	const FString MinTextureResolutionString("1");
	PropertyX->SetInstanceMetaData(TEXT("ClampMin"), *MinTextureResolutionString);
	PropertyX->SetInstanceMetaData(TEXT("UIMin"), *MinTextureResolutionString);
	PropertyY->SetInstanceMetaData(TEXT("ClampMin"), *MinTextureResolutionString);
	PropertyY->SetInstanceMetaData(TEXT("UIMin"), *MinTextureResolutionString);
}
