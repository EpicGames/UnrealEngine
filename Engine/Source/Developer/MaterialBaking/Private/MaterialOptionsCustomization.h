// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "IDetailCustomization.h"
#include "PropertyRestriction.h"

/** Property customization for Material Property entries */
class FPropertyEntryCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();
	FPropertyEntryCustomization();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
protected:
	void UpdateRestrictions(const int32 EntryIndex);
protected:
	/** Property restriction instance used for limiting EMaterialProperty selection */
	TSharedPtr<FPropertyRestriction> PropertyRestriction;
	class UMaterialOptions* CurrentOptions;
};

/** Detail customization for UMaterialOptions */
class FMaterialOptionsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance(int32 InNumLODs);
	FMaterialOptionsCustomization(int32 InNumLODs);

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
protected:
	int32 NumLODs;
};

static void AddTextureSizeClamping(TSharedPtr<IPropertyHandle> TextureSizeProperty);
