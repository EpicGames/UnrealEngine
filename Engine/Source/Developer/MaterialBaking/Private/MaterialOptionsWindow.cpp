// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialOptionsWindow.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

#include "IDetailsView.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"

#include "MaterialBakingStructures.h"
#include "MaterialOptions.h"
#include "MaterialOptionsCustomization.h"
#include "Misc/MessageDialog.h"

#define LOCTEXT_NAMESPACE "SMaterialOptions"

SMaterialOptions::SMaterialOptions() : bUserCancelled(true)
{

}

void SMaterialOptions::Construct(const FArguments& InArgs)
{
	WidgetWindow = InArgs._WidgetWindow;

	// Retrieve property editor module and create a SDetailsView
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = true;

	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	// Register instance property customization
	DetailsView->RegisterInstancedCustomPropertyLayout(UMaterialOptions::StaticClass(), FOnGetDetailCustomizationInstance::CreateLambda([=]() { return FMaterialOptionsCustomization::MakeInstance(InArgs._NumLODs); }));

	// Set provided objects on SDetailsView
	DetailsView->SetObjects(InArgs._SettingsObjects, true);
	
	this->ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.Padding(2)
		.MaxHeight(500.0f)
		[
			DetailsView->AsShared()
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(2)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(2)
			+ SUniformGridPanel::Slot(0, 0)
			[
				SAssignNew(ConfirmButton, SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("MaterialBakeOptionWindow_Import", "Confirm"))
				.OnClicked(this, &SMaterialOptions::OnConfirm)
			]
			+ SUniformGridPanel::Slot(1, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("MaterialBakeOptionWindow_Cancel", "Cancel"))
				.ToolTipText(LOCTEXT("MaterialBakeOptionWindow_Cancel_ToolTip", "Cancels baking out Material"))
				.OnClicked(this, &SMaterialOptions::OnCancel)
			]
		]
	];
}

FReply SMaterialOptions::OnConfirm()
{
	// Ensure the user has selected at least one LOD index
	if (GetMutableDefault<UMaterialOptions>()->LODIndices.Num() == 0)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("MaterialBake_SelectLODError", "Ensure that atleast one LOD index is selected."), LOCTEXT("MaterialBake_SelectLODErrorTitle", "Invalid options"));
	}
	else
	{
		bUserCancelled = false;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
	}	
	
	return FReply::Handled();
}

FReply SMaterialOptions::OnCancel()
{
	if (WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SMaterialOptions::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		return OnCancel();
	}

	return FReply::Unhandled();
}

bool SMaterialOptions::WasUserCancelled()
{
	return bUserCancelled;
}

#undef LOCTEXT_NAMESPACE //"SMaterialOptions"

