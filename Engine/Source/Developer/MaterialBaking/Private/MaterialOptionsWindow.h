// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "IDetailRootObjectCustomization.h"

struct FMaterialData;
class SButton;

/** Options window used to populate provided settings objects */
class SMaterialOptions : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMaterialOptions)
		: _WidgetWindow(), _NumLODs(1)
	{}
		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
		SLATE_ARGUMENT(int32, NumLODs)
		SLATE_ARGUMENT(TArray<TWeakObjectPtr<UObject>>, SettingsObjects)
	SLATE_END_ARGS()

public:
	SMaterialOptions();
	void Construct(const FArguments& InArgs);
	
	/** Begin SCompoundWidget overrides */
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	/** End SCompoundWidget overrides */

	/** Callbacks used for Confirm and Cancel buttons */
	FReply OnConfirm();
	FReply OnCancel();

	/** Returns whether or not the user cancelled the operation */
	bool WasUserCancelled();
private:
	/** Owning window this widget is part of */
	TWeakPtr< SWindow > WidgetWindow;
	/** Whether or not the cancel button was clicked by the user */
	bool bUserCancelled;
	/** Detailsview used to display SettingsObjects, and allowing user to change options */
	TSharedPtr<class IDetailsView> DetailsView;
	/** Shared ptr to Confirm button */
	TSharedPtr<SButton> ConfirmButton;
};