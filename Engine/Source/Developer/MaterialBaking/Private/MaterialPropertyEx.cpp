// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialPropertyEx.h"
#include "UObject/NameTypes.h"

const FMaterialPropertyEx FMaterialPropertyEx::ClearCoatBottomNormal(TEXT("ClearCoatBottomNormal"));
const FMaterialPropertyEx FMaterialPropertyEx::TransmittanceColor(TEXT("TransmittanceColor"));

FString FMaterialPropertyEx::ToString() const
{
	if (!IsCustomOutput())
	{
		const UEnum* Enum = StaticEnum<EMaterialProperty>();
		FName Name = Enum->GetNameByValue(Type);
		FString TrimmedName = Name.ToString();
		TrimmedName.RemoveFromStart(TEXT("MP_"), ESearchCase::CaseSensitive);
		return TrimmedName;
	}

	return CustomOutput.ToString();
}
