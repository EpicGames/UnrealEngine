// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialRenderItem.h"
#include "MaterialBakingStructures.h"

#include "EngineModule.h"
#include "SceneView.h"
#include "StaticMeshAttributes.h"
#include "DynamicMeshBuilder.h"
#include "MeshPassProcessor.h"
#include "CanvasRender.h"
#include "RHIStaticStates.h"
#include "UnrealClient.h"
#include "PrimitiveUniformShaderParametersBuilder.h"

#define SHOW_WIREFRAME_MESH 0

FMeshMaterialRenderItem::FMeshMaterialRenderItem(
	const FIntPoint& InTextureSize,
	const FMeshData* InMeshSettings,
	FDynamicMeshBufferAllocator* InDynamicMeshBufferAllocator)
	: MeshSettings(InMeshSettings)
	, TextureSize(InTextureSize)
	, MaterialRenderProxy(nullptr)
	, ViewFamily(nullptr)
	, bMeshElementDirty(true)
	, DynamicMeshBufferAllocator(InDynamicMeshBufferAllocator)
{
	GenerateRenderData();
	LCI = new FMeshRenderInfo(InMeshSettings->LightMap, nullptr, nullptr, InMeshSettings->LightmapResourceCluster);
}

bool FMeshMaterialRenderItem::Render_RenderThread(FCanvasRenderContext& RenderContext, FMeshPassProcessorRenderState& DrawRenderState, const FCanvas* Canvas)
{
	checkSlow(ViewFamily && MeshSettings && MaterialRenderProxy);
	// current render target set for the canvas
	const FRenderTarget* CanvasRenderTarget = Canvas->GetRenderTarget();
	const FIntRect ViewRect(FIntPoint(0, 0), CanvasRenderTarget->GetSizeXY());

	// make a temporary view
	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.ViewFamily = ViewFamily;
	ViewInitOptions.SetViewRectangle(ViewRect);
	ViewInitOptions.ViewOrigin = FVector::ZeroVector;
	ViewInitOptions.ViewRotationMatrix = FMatrix::Identity;
	ViewInitOptions.ProjectionMatrix = Canvas->GetTransformStack().Top().GetMatrix();
	ViewInitOptions.BackgroundColor = FLinearColor::Black;
	ViewInitOptions.OverlayColor = FLinearColor::White;

	FSceneView View(ViewInitOptions);
	View.FinalPostProcessSettings.bOverride_IndirectLightingIntensity = 1;
	View.FinalPostProcessSettings.IndirectLightingIntensity = 0.0f;

	if (Vertices.Num() && Indices.Num())
	{
		FMeshPassProcessorRenderState LocalDrawRenderState;

		// disable depth test & writes
		LocalDrawRenderState.SetBlendState(TStaticBlendState<CW_RGBA>::GetRHI());
		LocalDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());

		QueueMaterial(RenderContext, LocalDrawRenderState, &View);
	}

	return true;
}

bool FMeshMaterialRenderItem::Render_GameThread(const FCanvas* Canvas, FCanvasRenderThreadScope& RenderScope)
{
	RenderScope.EnqueueRenderCommand(
		[this, Canvas](FCanvasRenderContext& RenderContext)
		{
			// Render_RenderThread uses its own render state
			FMeshPassProcessorRenderState DummyRenderState;
			Render_RenderThread(RenderContext, DummyRenderState, Canvas);
		}
	);

	return true;
}

void FMeshMaterialRenderItem::GenerateRenderData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshMaterialRenderItem::GenerateRenderData)

	// Reset array without resizing
	Vertices.SetNum(0, EAllowShrinking::No);
	Indices.SetNum(0, EAllowShrinking::No);
	if (MeshSettings->MeshDescription)
	{
		// Use supplied FMeshDescription data to populate render data
		PopulateWithMeshData();
	}
	else
	{
		// Use simple rectangle
		PopulateWithQuadData();
	}

	bMeshElementDirty = true;
}

FMeshMaterialRenderItem::~FMeshMaterialRenderItem()
{
	// Send the release of the buffers to the render thread
	ENQUEUE_RENDER_COMMAND(ReleaseResources)(
		[ToRelease = MoveTemp(MeshBuilderResources)](FRHICommandListImmediate& RHICmdList) {}
	);
}

void FMeshMaterialRenderItem::QueueMaterial(FCanvasRenderContext& RenderContext, FMeshPassProcessorRenderState& DrawRenderState, const FSceneView* View)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshMaterialRenderItem::QueueMaterial)

	if (bMeshElementDirty)
	{
		MeshBuilderResources.Clear();
		FDynamicMeshBuilder DynamicMeshBuilder(View->GetFeatureLevel(), MAX_STATIC_TEXCOORDS, MeshSettings->LightMapIndex, false, DynamicMeshBufferAllocator);
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(CopyData);
			DynamicMeshBuilder.AddVertices(Vertices);
			DynamicMeshBuilder.AddTriangles(Indices);
		}

		const FPrimitiveData DefaultPrimitiveData;
		const FPrimitiveData& PrimitiveData = MeshSettings->PrimitiveData.Get(DefaultPrimitiveData);
		FPrimitiveUniformShaderParameters PrimitiveParams = FPrimitiveUniformShaderParametersBuilder{}
			.Defaults()
				.LocalToWorld(PrimitiveData.LocalToWorld)
				.ActorWorldPosition(PrimitiveData.ActorPosition)
				.WorldBounds(PrimitiveData.WorldBounds)
				.LocalBounds(PrimitiveData.LocalBounds)
				.PreSkinnedLocalBounds(PrimitiveData.PreSkinnedLocalBounds)
				.CustomPrimitiveData(PrimitiveData.CustomPrimitiveData)
				.ReceivesDecals(false)
				.OutputVelocity(false)
			.Build();

		DynamicMeshBuilder.GetMeshElement(PrimitiveParams, MaterialRenderProxy, SDPG_Foreground, true, 0, MeshBuilderResources, MeshElement);

		check(MeshBuilderResources.IsValidForRendering());
		bMeshElementDirty = false;
	}

	MeshElement.MaterialRenderProxy = MaterialRenderProxy;

	LCI->CreatePrecomputedLightingUniformBuffer_RenderingThread(View->GetFeatureLevel());
	MeshElement.LCI = LCI;

#if SHOW_WIREFRAME_MESH
	MeshElement.bWireframe = true;
#endif

	const int32 NumTris = FMath::TruncToInt((float)Indices.Num() / 3);
	if (NumTris == 0)
	{
		// there's nothing to do here
		return;
	}

	// Bake the material out to a tile
	GetRendererModule().DrawTileMesh(RenderContext, DrawRenderState, *View, MeshElement, false /*bIsHitTesting*/, FHitProxyId());
}

void FMeshMaterialRenderItem::PopulateWithQuadData()
{
	// Pre-transform all vertices with the inverse of LocalToWorld to negate its effect during material baking 
	const FMatrix44f WorldToLocal = MeshSettings->PrimitiveData.IsSet() ? FMatrix44f(MeshSettings->PrimitiveData->LocalToWorld.Inverse()) : FMatrix44f::Identity;
	
	Vertices.Empty(4);
	Indices.Empty(6);

	const float OffsetU = MeshSettings->TextureCoordinateBox.Min.X;
	const float OffsetV = MeshSettings->TextureCoordinateBox.Min.Y;
	const float SizeU = MeshSettings->TextureCoordinateBox.Max.X - MeshSettings->TextureCoordinateBox.Min.X;
	const float SizeV = MeshSettings->TextureCoordinateBox.Max.Y - MeshSettings->TextureCoordinateBox.Min.Y;
	const float ScaleX = TextureSize.X;
	const float ScaleY = TextureSize.Y;

	// add vertices
	for (int32 VertIndex = 0; VertIndex < 4; VertIndex++)
	{
		FDynamicMeshVertex* Vert = new(Vertices)FDynamicMeshVertex();
		const int32 X = VertIndex & 1;
		const int32 Y = (VertIndex >> 1) & 1;
		Vert->Position = WorldToLocal.TransformPosition(FVector3f(ScaleX * X, ScaleY * Y, 0));
		FVector3f TangentX = WorldToLocal.TransformVector(FVector3f(1, 0, 0));
		FVector3f TangentZ = WorldToLocal.TransformVector(FVector3f(0, 1, 0));
		FVector3f TangentY = WorldToLocal.TransformVector(FVector3f(0, 0, 1));
		Vert->SetTangents(TangentX, TangentZ, TangentY);
		FMemory::Memzero(&Vert->TextureCoordinate, sizeof(Vert->TextureCoordinate));
		for (int32 TexcoordIndex = 0; TexcoordIndex < MAX_STATIC_TEXCOORDS; TexcoordIndex++)
		{
			Vert->TextureCoordinate[TexcoordIndex].Set(OffsetU + SizeU * X, OffsetV + SizeV * Y);
		}
		Vert->Color = FColor::White;
	}

	// add indices
	static const uint32 TriangleIndices[6] = { 0, 2, 1, 2, 3, 1 };
	Indices.Append(TriangleIndices, 6);
}

void FMeshMaterialRenderItem::PopulateWithMeshData()
{
	// Pre-transform all vertices with the inverse of LocalToWorld to negate its effect during material baking 
	const FMatrix44f WorldToLocal = MeshSettings->PrimitiveData.IsSet() ? FMatrix44f(MeshSettings->PrimitiveData->LocalToWorld.Inverse()) : FMatrix44f::Identity;

	const FMeshDescription* RawMesh = MeshSettings->MeshDescription;

	FStaticMeshConstAttributes Attributes(*RawMesh);
	TArrayView<const FVector3f> VertexPositions = Attributes.GetVertexPositions().GetRawArray();
	TArrayView<const FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals().GetRawArray();
	TArrayView<const FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents().GetRawArray();
	TArrayView<const float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns().GetRawArray();
	TArrayView<const FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors().GetRawArray();

	TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
	const int32 NumVerts = RawMesh->Vertices().Num();

	// reserve renderer data
	Vertices.Empty(NumVerts);
	Indices.Empty(NumVerts >> 1);

	// When using arbitrary mesh data (rather than a simple quad), TextureCoordinateBox has to be applied to XY position:
	const float ScaleX = TextureSize.X / (MeshSettings->TextureCoordinateBox.Max.X - MeshSettings->TextureCoordinateBox.Min.X);
	const float ScaleY = TextureSize.Y / (MeshSettings->TextureCoordinateBox.Max.Y - MeshSettings->TextureCoordinateBox.Min.Y);
	const float OffsetX = -MeshSettings->TextureCoordinateBox.Min.X * ScaleX;
	const float OffsetY = -MeshSettings->TextureCoordinateBox.Min.Y * ScaleY;

	const static int32 VertexPositionStoredUVChannel = 6;
	// count number of texture coordinates for this mesh
	const int32 NumTexcoords = [&]()
	{
		return FMath::Min(VertexInstanceUVs.GetNumChannels(), VertexPositionStoredUVChannel);
	}();

	// check if we should use NewUVs or original UV set
	const bool bUseNewUVs = MeshSettings->CustomTextureCoordinates.Num() > 0;
	if (bUseNewUVs)
	{
		check(MeshSettings->CustomTextureCoordinates.Num() == VertexInstanceUVs.GetNumElements() && VertexInstanceUVs.GetNumChannels() > MeshSettings->TextureCoordinateIndex);
	}

	// add vertices
	int32 VertIndex = 0;
	int32 FaceIndex = 0;
	for(const FTriangleID& TriangleID : RawMesh->Triangles().GetElementIDs())
	{
		const FPolygonGroupID PolygonGroupID = RawMesh->GetTrianglePolygonGroup(TriangleID);
		if (MeshSettings->MaterialIndices.Contains(PolygonGroupID.GetValue()))
		{
			const int32 NUM_VERTICES = 3;
			for (int32 Corner = 0; Corner < NUM_VERTICES; Corner++)
			{
				// Swap vertices order if mesh is mirrored
				const int32 CornerIdx = !MeshSettings->bMirrored ? Corner : NUM_VERTICES - Corner - 1;

				const int32 SrcVertIndex = FaceIndex * NUM_VERTICES + CornerIdx;
				const FVertexInstanceID SrcVertexInstanceID = RawMesh->GetTriangleVertexInstance(TriangleID, Corner);
				const FVertexID SrcVertexID = RawMesh->GetVertexInstanceVertex(SrcVertexInstanceID);

				// add vertex
				FDynamicMeshVertex* Vert = new(Vertices)FDynamicMeshVertex();
				if (!bUseNewUVs)
				{
					// compute vertex position from original UV
					const FVector2D& UV = FVector2D(VertexInstanceUVs.Get(SrcVertexInstanceID, MeshSettings->TextureCoordinateIndex));
					Vert->Position = WorldToLocal.TransformPosition(FVector3f(OffsetX + UV.X * ScaleX, OffsetY + UV.Y * ScaleY, 0));
				}
				else
				{
					const FVector2D& UV = MeshSettings->CustomTextureCoordinates[SrcVertIndex];
					Vert->Position = WorldToLocal.TransformPosition(FVector3f(OffsetX + UV.X * ScaleX, OffsetY + UV.Y * ScaleY, 0));
				}
				FVector3f TangentX = WorldToLocal.TransformVector(VertexInstanceTangents[SrcVertexInstanceID]);
				FVector3f TangentZ = WorldToLocal.TransformVector(VertexInstanceNormals[SrcVertexInstanceID]);
				FVector3f TangentY = FVector3f::CrossProduct(TangentZ, TangentX).GetSafeNormal() * VertexInstanceBinormalSigns[SrcVertexInstanceID];
				Vert->SetTangents(TangentX, TangentY, TangentZ);
				for (int32 TexcoordIndex = 0; TexcoordIndex < NumTexcoords; TexcoordIndex++)
				{
					Vert->TextureCoordinate[TexcoordIndex] = VertexInstanceUVs.Get(SrcVertexInstanceID, TexcoordIndex);
				}

				if (NumTexcoords < VertexPositionStoredUVChannel)
				{
					for (int32 TexcoordIndex = NumTexcoords; TexcoordIndex < VertexPositionStoredUVChannel; TexcoordIndex++)
					{
						Vert->TextureCoordinate[TexcoordIndex] = Vert->TextureCoordinate[FMath::Max(NumTexcoords - 1, 0)];
					}
				}
				// Store original vertex positions in texture coordinate data
				Vert->TextureCoordinate[6].X = VertexPositions[SrcVertexID].X;
				Vert->TextureCoordinate[6].Y = VertexPositions[SrcVertexID].Y;
				Vert->TextureCoordinate[7].X = VertexPositions[SrcVertexID].Z;
				Vert->TextureCoordinate[7].Y = 0.0f;

				Vert->Color = FLinearColor(VertexInstanceColors[SrcVertexInstanceID]).ToFColor(true);
				// add index
				Indices.Add(VertIndex);
				VertIndex++;
			}
		}
		FaceIndex++;
	}
}
