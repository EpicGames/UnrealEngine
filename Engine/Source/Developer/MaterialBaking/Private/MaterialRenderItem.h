// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CanvasTypes.h"
#include "MaterialRenderItemData.h"
#include "MeshBuilderOneFrameResources.h"
#include "DynamicMeshBuilder.h"

class FSceneViewFamily;
class FMaterialRenderProxy;
class FSceneView;
class FRHICommandListImmediate;
struct FMaterialData;
struct FMeshData;
struct FMeshPassProcessorRenderState;

// This will hold onto the resources until not needed anymore.
// Move constructor makes it easier to send the destruction to another thread (render thread).
class FMeshBuilderResources : public FMeshBuilderOneFrameResources
{
public:
	FMeshBuilderResources() = default;
	FMeshBuilderResources(FMeshBuilderResources&& Other)
	{
		if (this != &Other)
		{
			VertexBuffer = Other.VertexBuffer;
			IndexBuffer = Other.IndexBuffer;
			VertexFactory = Other.VertexFactory;
			PrimitiveUniformBuffer = Other.PrimitiveUniformBuffer;

			Other.VertexBuffer = nullptr;
			Other.IndexBuffer = nullptr;
			Other.VertexFactory = nullptr;
			Other.PrimitiveUniformBuffer = nullptr;
		}
	}

	void Clear()
	{
		FMeshBuilderOneFrameResources::~FMeshBuilderOneFrameResources();
		VertexBuffer = nullptr;
		IndexBuffer = nullptr;
		VertexFactory = nullptr;
		PrimitiveUniformBuffer = nullptr;
	}
};

class FMeshMaterialRenderItem : public FCanvasBaseRenderItem
{
public:
	FMeshMaterialRenderItem(const FIntPoint& InTextureSize, const FMeshData* InMeshSettings, FDynamicMeshBufferAllocator* InDynamicMeshBufferAllocator = nullptr);
	virtual ~FMeshMaterialRenderItem();

	/** Begin FCanvasBaseRenderItem overrides */
	virtual bool Render_RenderThread(FCanvasRenderContext& RenderContext, FMeshPassProcessorRenderState& DrawRenderState, const FCanvas* Canvas) final;
	virtual bool Render_GameThread(const FCanvas* Canvas, FCanvasRenderThreadScope& RenderScope) final;
	/** End FCanvasBaseRenderItem overrides */

	/** Populate vertices and indices according to available mesh data and otherwise uses simple quad */
	void GenerateRenderData();
protected:
	/** Enqueues the current material to be rendered */
	void QueueMaterial(FCanvasRenderContext& RenderContext, FMeshPassProcessorRenderState& DrawRenderState, const FSceneView* View);
	
	/** Helper functions to populate render data using either mesh data or a simple quad */
	void PopulateWithQuadData();
	void PopulateWithMeshData();
public:
	/** Mesh settings to use while baking out the material */
	const FMeshData* MeshSettings;
	/** The texture size to use while baking */
	FIntPoint TextureSize;
	/** Material render proxy (material/shader) to use while baking */
	FMaterialRenderProxy* MaterialRenderProxy;
	/** Vertex and index data representing the mesh or a quad */
	TArray<FDynamicMeshVertex> Vertices;
	TArray<uint32> Indices;
	/** Light cache interface object to simulate lightmap behavior in case the material used prebaked ambient occlusion */
	FLightCacheInterface* LCI;
	/** View family to use while baking */
	FSceneViewFamily* ViewFamily;
private:
	FMeshBatch MeshElement;
	bool bMeshElementDirty;
	FMeshBuilderResources MeshBuilderResources;
	FDynamicMeshBufferAllocator* DynamicMeshBufferAllocator;
};
