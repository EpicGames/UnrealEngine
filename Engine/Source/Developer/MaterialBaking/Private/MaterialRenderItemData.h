// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LightMapHelpers.h"
#include "Components.h"
#include "LocalVertexFactory.h"
#include "SceneManagement.h"

/** Simple implementation for light cache to simulated lightmap behaviour (used for accessing prebaked ambient occlusion values) */
class FMeshRenderInfo : public FLightCacheInterface
{
public:
	FMeshRenderInfo(const FLightMap* InLightMap, const FShadowMap* InShadowMap, FUniformBufferRHIRef Buffer, const FLightmapResourceCluster* InLightmapResourceCluster)
		: FLightCacheInterface()
	{
		SetLightMap(InLightMap);
		SetShadowMap(InShadowMap);
		SetPrecomputedLightingBuffer(Buffer);
		if (InLightmapResourceCluster)
		{
			SetResourceCluster(InLightmapResourceCluster);
		}
	}

	virtual FLightInteraction GetInteraction(const class FLightSceneProxy* LightSceneProxy) const override
	{
		return LIT_CachedLightMap;
	}
};