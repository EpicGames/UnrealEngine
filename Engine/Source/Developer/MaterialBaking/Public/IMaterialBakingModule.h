// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "MaterialPropertyEx.h"
#include "PixelFormat.h"
#include "SceneTypes.h"

class UTextureRenderTarget2D;
class UMaterialOptions;
class UMaterialInterface;

class FExportMaterialProxy;
class FTextureRenderTargetResource;

struct FMaterialData; 
struct FMaterialDataEx;
struct FMeshData;
struct FBakeOutput;
struct FBakeOutputEx;

class IMaterialBakingModule : public IModuleInterface
{
public:
	/** Bakes out material properties according to MaterialSettings using MeshSettings and stores the output in Output */
	virtual void BakeMaterials(const TArray<FMaterialData*>& MaterialSettings, const TArray<FMeshData*>& MeshSettings, TArray<FBakeOutput>& Output) = 0;

	/** Bakes out material properties according to extended MaterialSettings using MeshSettings and stores the output in Output */
	virtual void BakeMaterials(const TArray<FMaterialDataEx*>& MaterialSettings, const TArray<FMeshData*>& MeshSettings, TArray<FBakeOutputEx>& Output) = 0;

	/** Bakes out material properties according to MaterialSettings using MeshSettings and stores the output in Output */
	virtual void BakeMaterials(const TArray<FMaterialData*>& MaterialSettings, const TArray<FMeshData*>& MeshSettings, FBakeOutput& Output) = 0;

	/** Bakes out material properties according to extended MaterialSettings using MeshSettings and stores the output in Output */
	virtual void BakeMaterials(const TArray<FMaterialDataEx*>& MaterialSettings, const TArray<FMeshData*>& MeshSettings, FBakeOutputEx& Output) = 0;

	/** Promps a slate window to allow the user to populate specific material baking settings used while baking out materials */
	virtual  bool SetupMaterialBakeSettings(TArray<TWeakObjectPtr<UObject>>& OptionObjects, int32 NumLODs) = 0;

	/** Outputs true HDR version of emissive color */
	virtual void SetEmissiveHDR(bool bHDR) = 0;

	/** Bakes all material properties to linear textures, except for colors */
	virtual void SetLinearBake(bool bCorrectLinear) = 0;

	/** Returns whether a specific material property is baked to a linear texture or not */
	virtual bool IsLinearBake(FMaterialPropertyEx Property) = 0;

	/** Obtain a CRC than can help trigger a rebake if code/global settings impacting the bake result change */
	virtual uint32 GetCRC() const = 0;
};
