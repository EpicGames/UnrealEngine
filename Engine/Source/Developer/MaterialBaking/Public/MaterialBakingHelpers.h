// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/Color.h"

#define UE_API MATERIALBAKING_API

struct FColor;

class FMaterialBakingHelpers
{
public:
	/**
	 * Applies a box blur to magenta pixels found in given texture represented by InOutPixels using non-background pixels, this creates a smear across the magenta/filled pixels.
	 * @param  InOutPixels     The image to apply the smear to
	 * @param  ImageWidth      The width of the image in pixels
	 * @param  ImageHeight     The height of the image in pixels
	 * @param  MaxIterations   The max distance in pixels to smear the edges of the texture. When set to -1 this value is the max of the width and height of the buffer
	 * @param  BackgroundColor The pixel color considered background in the image, and that can be smeared into
	 */
	static UE_API void PerformUVBorderSmear(TArray<FColor>& InOutPixels, int32 ImageWidth, int32 ImageHeight, int32 MaxIterations = -1, FColor BackgroundColor = FColor::Magenta);

	/**
	 * Shrinks the array to a single pixel if the entire output is monochrome. Params will be adjusted accordingly when this occurs.
	 * It's useful to perform this before the border smear as a single non-magenta pixel value would have led to a monochrome image post-smear anyway,
	 * so the actual smearing process can be skipped.
	 * @param  InOutPixels     The image to shrink
	 * @param  ImageWidth      The width of the image in pixels
	 * @param  ImageHeight     The height of the image in pixels
	 */
	static UE_API void PerformShrinking(TArray<FColor>& InOutPixels, int32& OnOutImageWidth, int32& InOutImageHeight, FColor BackgroundColor = FColor::Magenta);

	/**
	 * Applies a box blur to magenta pixels found in given texture represented by InOutPixels using non-background pixels, this creates a smear across the magenta/filled pixels.
	 * @param  InOutPixels      The image to apply the smear to
	 * @param  InOutImageWidth  The width of the image in pixels
	 * @param  InOutImageHeight The height of the image in pixels
	 * @param  BackgroundColor  The pixel color considered background in the image, and that can be smeared into
	 *
	 * @note   This version is allowed to shrink the array to a single pixel if the entire output is monochrome. Params will be adjusted accordingly when this occurs.
	 */
	static UE_API void PerformUVBorderSmearAndShrink(TArray<FColor>& InOutPixels, int32& InOutImageWidth, int32& InOutImageHeight, FColor BackgroundColor = FColor::Magenta);
};

#undef UE_API
