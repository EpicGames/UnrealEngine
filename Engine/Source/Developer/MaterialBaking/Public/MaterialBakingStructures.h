// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/IntPoint.h"
#include "Components/MeshComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "SceneTypes.h"
#include "LightMap.h"
#include "MaterialPropertyEx.h"

class UMaterialInterface;
struct FMeshDescription;

/** Structure containing information about the material which is being baked out */
struct FMaterialData
{
	/** Material to bake out */
	UMaterialInterface* Material = nullptr;

	/** Properties and the texture size at which they should be baked out */
	TMap<EMaterialProperty, FIntPoint> PropertySizes;

	/** Whether to smear borders after baking */
	bool bPerformBorderSmear = true;

	/** Whether to try to shrink the color values to a single non-magenta color value after baking */
	bool bPerformShrinking = true;

	/** Whether to transform normals from world-space to tangent-space (does nothing if material already uses tangent-space normals) */
	bool bTangentSpaceNormal = false;

	/** Blend mode to use when baking, allowing for example detection of overlapping UVs */
	EBlendMode BlendMode = BLEND_Opaque;

	/** Background color used to initially fill the output texture and used for border smear */
	FColor BackgroundColor = FColor::Magenta;
};

/** Structure containing extended information about the material and properties which is being baked out */
struct FMaterialDataEx
{
	/** Material to bake out */
	UMaterialInterface* Material = nullptr;

	/** Extended properties and the texture size at which they should be baked out */
	TMap<FMaterialPropertyEx, FIntPoint> PropertySizes;

	/** Whether to smear borders after baking */
	bool bPerformBorderSmear = true;

	/** Whether to try to shrink the color values to a single non-magenta color value after baking */
	bool bPerformShrinking = true;

	/** Whether to transform normals from world-space to tangent-space (does nothing if material already uses tangent-space normals) */
	bool bTangentSpaceNormal = false;

	/** Blend mode to use when baking, allowing for example detection of overlapping UVs */
	EBlendMode BlendMode = BLEND_Opaque;

	/** Background color used to initially fill the output texture and used for border smear */
	FColor BackgroundColor = FColor::Magenta;
};

/** Structure containing primitive information (regarding a mesh or mesh component) that is accessible through material expressions */
struct FPrimitiveData
{
	FPrimitiveData(const FBoxSphereBounds& LocalBounds = FBoxSphereBounds(ForceInitToZero))
		: LocalToWorld(FMatrix::Identity)
		, ActorPosition(ForceInitToZero)
		, WorldBounds(LocalBounds)
		, LocalBounds(LocalBounds)
		, PreSkinnedLocalBounds(LocalBounds)
		, CustomPrimitiveData(nullptr)
	{}

	FPrimitiveData(const UStaticMesh* StaticMesh)
		: FPrimitiveData(StaticMesh->GetBounds())
	{}

	FPrimitiveData(const USkeletalMesh* SkeletalMesh)
		: FPrimitiveData(SkeletalMesh->GetBounds())
	{}

	FPrimitiveData(const UMeshComponent* MeshComponent)
		: LocalToWorld(MeshComponent->GetRenderMatrix())
		, ActorPosition(MeshComponent->GetActorPositionForRenderer())
		, WorldBounds(MeshComponent->Bounds)
		, LocalBounds(MeshComponent->GetLocalBounds())
		, CustomPrimitiveData(&MeshComponent->GetCustomPrimitiveData())
	{
		if (const USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(MeshComponent))
		{
			SkinnedMeshComponent->GetPreSkinnedLocalBounds(PreSkinnedLocalBounds);
		}
		else
		{
			PreSkinnedLocalBounds = LocalBounds;
		}
	}

	/** The mesh component's local-to-world transform */
	FMatrix LocalToWorld;
	/** The actor's location in world-space */
	FVector ActorPosition;
	/** The mesh component's bounds in world-space */
	FBoxSphereBounds WorldBounds;
	/** The mesh component's bounds in local-space */
	FBoxSphereBounds LocalBounds;
	/** The mesh component's pre-skinning bounds in local-space */
	FBoxSphereBounds PreSkinnedLocalBounds;
	/** The mesh component's custom primitive data */
	const FCustomPrimitiveData* CustomPrimitiveData;
};

struct FMeshData
{
	/** Ptr to raw mesh data to use for baking out the material data, if nullptr a standard quad is used */
	const FMeshDescription* MeshDescription = nullptr;

	/** Ptr to original static mesh this mesh data came from */
	const UStaticMesh* Mesh = nullptr;

	/** Transform determinant used to detect mirroring */
	bool bMirrored = false;

	/** A hash of the vertex color buffer for the rawmesh */
	uint32 VertexColorHash = 0;

	/** Material indices to test the Raw Mesh data against, ensuring we only bake out triangles which use the currently baked out material */
	TArray<int32> MaterialIndices;

	/** Set of custom texture coordinates which ensure that the material is baked out with unique/non-overlapping positions */
	TArray<FVector2D> CustomTextureCoordinates;

	/** Box which's space contains the UV coordinates used to bake out the material */
	FBox2D TextureCoordinateBox = FBox2D(FVector2D(0.0f, 0.0f), FVector2D(1.0f, 1.0f));
	
	/** Specific texture coordinate index to use as texture coordinates to bake out the material (is overruled if CustomTextureCoordinates contains any data) */
	int32 TextureCoordinateIndex = 0;

	/** Light map index used to retrieve the light-map UVs from RawMesh */
	int32 LightMapIndex = 0;

	/** Reference to the lightmap texture part of the level in the currently being baked out mesh instance data is resident */
	FLightMapRef LightMap = nullptr;

	/** Pointer to the LightmapResourceCluster to be passed on the the LightCacheInterface when baking */
	const FLightmapResourceCluster* LightmapResourceCluster = nullptr;

	/** Optional primitive data that is accessible through material expressions */
	TOptional<FPrimitiveData> PrimitiveData;
};

/** Structure containing data being processed while baking out materials*/
struct FBakeOutput
{
	/** Contains the resulting texture data for baking out a material's property */
	TMap<EMaterialProperty, TArray<FColor>> PropertyData;

	/** Contains the resulting texture size for baking out a material's property */
	TMap<EMaterialProperty, FIntPoint> PropertySizes;

	/** Contains the resulting HDR texture data for baking out a material's property, may be empty */
	TMap<EMaterialProperty, TArray<FFloat16Color>> HDRPropertyData;

	/** Wether or not a baked texture data is using a linear color or sRGB encoding. */
	TMap<EMaterialProperty, bool> PropertyIsLinearColor;

	/** Scale used to allow having wide ranges of emissive values in the source materials, the final proxy material will use this value to scale the emissive texture's pixel values */
	float EmissiveScale = 1.0f;
};

/** Structure containing extended data being processed while baking out materials*/
struct FBakeOutputEx
{
	/** Contains the resulting texture data for baking out a extened material's property */
	TMap<FMaterialPropertyEx, TArray<FColor>> PropertyData;

	/** Contains the resulting texture size for baking out a extened material's property */
	TMap<FMaterialPropertyEx, FIntPoint> PropertySizes;

	/** Contains the resulting HDR texture data for baking out a material's property, may be empty */
	TMap<FMaterialPropertyEx, TArray<FFloat16Color>> HDRPropertyData;

	/** Wether or not a baked texture data is using a linear color or sRGB encoding. */
	TMap<FMaterialPropertyEx, bool> PropertyIsLinearColor;

	/** Scale used to allow having wide ranges of emissive values in the source materials, the final proxy material will use this value to scale the emissive texture's pixel values */
	float EmissiveScale = 1.0f;
};
