// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneTypes.h"
#include "Engine/EngineTypes.h"

#include "MaterialOptions.generated.h"

/** Enum to define different types of baking materials */
UENUM()
enum class EMaterialBakeMethod : uint8
{
	IndividualMaterial UMETA(DisplayName = "Bake out Materials Individually"),
	AtlasMaterial UMETA(DisplayName = "Combine Materials into Atlassed Material"),
	BinnedMaterial UMETA(DisplayName = "Combine Materials into Binned Material")
};

/** Structure to represent a single property the user wants to bake out for a given set of materials */
USTRUCT(Blueprintable)
struct FPropertyEntry
{
	GENERATED_BODY()

	FPropertyEntry()
		: Property(MP_EmissiveColor)
		, bUseCustomSize(false)
		, CustomSize(0, 0)
		, bUseConstantValue(false)
		, ConstantValue(0.0f)
	{}

	FPropertyEntry(EMaterialProperty InProperty)
		: Property(InProperty)
		, bUseCustomSize(false)
		, CustomSize(0, 0)
		, bUseConstantValue(false)
		, ConstantValue(0.0f)
	{}

	/** Property which should be baked out */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Property, meta = (ExposeOnSpawn))
	TEnumAsByte<EMaterialProperty> Property;

	/** Whether or not to use the value of custom size for the output texture */
	UPROPERTY(EditAnywhere, Category = Property, BlueprintReadWrite, meta = (InlineEditConditionToggle, ExposeOnSpawn))
	bool bUseCustomSize;

	/** Defines the size of the output textures for the baked out material properties */
	UPROPERTY(EditAnywhere, Category = Property, BlueprintReadWrite, meta = (EditCondition = bUseCustomSize, ExposeOnSpawn, ClampMin = "1", UIMin = "1"))
	FIntPoint CustomSize;

	/** Wheter or not to use Constant Value as the final 'baked out' value for the this property */
	UPROPERTY(EditAnywhere, Category = Property, BlueprintReadWrite, meta = (InlineEditConditionToggle, ExposeOnSpawn))
	bool bUseConstantValue;

	/** Defines the value representing this property in the final proxy material */
	UPROPERTY(EditAnywhere, Category = Property, BlueprintReadWrite, meta = (EditCondition = bUseConstantValue, ExposeOnSpawn))
	float ConstantValue;
};

/** Options object to define what and how a material should be baked out */
UCLASS(MinimalAPI, config = Editor, Blueprintable)
class UMaterialOptions : public UObject 
{
	GENERATED_BODY()

public:

	UMaterialOptions()
		: TextureSize(128, 128), bUseMeshData(false), bUseSpecificUVIndex(false), TextureCoordinateIndex(0)
	{
		Properties.Add(MP_BaseColor);
		LODIndices.Add(0);
	}

	/** Properties which are supposed to be baked out for the material(s) */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category= MaterialBakeSettings, meta = (ExposeOnSpawn))
	TArray<FPropertyEntry> Properties;

	/** Size of the final texture(s) containing the baked out property data */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = MaterialBakeSettings, meta = (ExposeOnSpawn, ClampMin = "1", UIMin = "1"))
	FIntPoint TextureSize;

	/** LOD indices for which the materials should be baked out */
	UPROPERTY(BlueprintReadWrite, Category = MeshSettings, meta = (ExposeOnSpawn))
	TArray<int32> LODIndices;

	/** Determines whether to not allow usage of the source mesh data while baking out material properties */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = MeshSettings, meta = (ExposeOnSpawn))
	bool bUseMeshData;

	/** Flag whether or not the value of TextureCoordinateIndex should be used while baking out material properties */
	UPROPERTY(EditAnywhere, Category = MeshSettings, BlueprintReadWrite, meta = (InlineEditConditionToggle, EditCondition = bUseMeshData, ExposeOnSpawn))
	bool bUseSpecificUVIndex;

	/** Specific texture coordinate which should be used to while baking out material properties as the positions stream */
	UPROPERTY(EditAnywhere, Category = MeshSettings, BlueprintReadWrite, meta = (EditCondition = bUseSpecificUVIndex, ExposeOnSpawn))
	int32 TextureCoordinateIndex;
};

/** Asset bake options object */
UCLASS(MinimalAPI, Config = Editor, Blueprintable)
class UAssetBakeOptions : public UObject
{
	GENERATED_BODY()
public:

	UAssetBakeOptions()
	{
	}
};

/** Material merge options object */
UCLASS(MinimalAPI, Config = Editor, Blueprintable)
class UMaterialMergeOptions: public UObject
{
	GENERATED_BODY()
public:

	UMaterialMergeOptions() : Method(EMaterialBakeMethod::IndividualMaterial), BlendMode(EBlendMode::BLEND_Opaque)
	{
	}

	/** Method used to bake out the materials, hidden for now */
	UPROPERTY(/*EditAnywhere, BlueprintReadWrite, config, Category = MeshSettings, meta = (ExposeOnSpawn)*/)
	EMaterialBakeMethod Method;

	/** Blend mode for the final proxy material(s) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = OutputMaterialSettings, meta = (ExposeOnSpawn))
	TEnumAsByte<EBlendMode> BlendMode;
};
