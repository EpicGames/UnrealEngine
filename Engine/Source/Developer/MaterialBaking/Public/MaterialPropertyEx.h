// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "SceneTypes.h"
#include "Templates/TypeHash.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

#define UE_API MATERIALBAKING_API

/** Structure extending EMaterialProperty to allow detailed information about custom output */
struct FMaterialPropertyEx
{
	FMaterialPropertyEx(EMaterialProperty Type = MP_MAX, const FName& CustomOutput = NAME_None)
		: Type(Type)
		, CustomOutput(CustomOutput)
	{}

	FMaterialPropertyEx(const FName& CustomOutput)
		: Type(MP_CustomOutput)
		, CustomOutput(CustomOutput)
	{}

	FMaterialPropertyEx(const TCHAR* CustomOutput)
		: Type(MP_CustomOutput)
		, CustomOutput(CustomOutput)
	{}

	FORCEINLINE bool IsCustomOutput() const
	{
		return Type == MP_CustomOutput;
	}

	FORCEINLINE bool operator ==(const FMaterialPropertyEx& Other) const
	{
		return Type == Other.Type && (!IsCustomOutput() || CustomOutput == Other.CustomOutput);
	}

	FORCEINLINE bool operator !=(const FMaterialPropertyEx& Other) const
	{
		return !(*this == Other);
	}

	friend FORCEINLINE uint32 GetTypeHash(const FMaterialPropertyEx& Other)
	{
		return !Other.IsCustomOutput() ? GetTypeHash(Other.Type) : GetTypeHash(Other.CustomOutput);
	}

	UE_API FString ToString() const;

	/** The material property */
	EMaterialProperty Type;

	/** The name of a specific custom output. Only used if property is MP_CustomOutput */
	FName CustomOutput;

	/** The identifier for custom output Clear Coat Bottom Normal (used by shading model Clear Coat) */
	static UE_API const FMaterialPropertyEx ClearCoatBottomNormal;

	/** The identifier for custom output Transmittance Color (used by shading model Thin Translucent) */
	static UE_API const FMaterialPropertyEx TransmittanceColor;
};

#undef UE_API
